// Copyright Epic Games, Inc. All Rights Reserved.

#include "Characters/Text3DCharacterBase.h"

#include "Text3DComponent.h"

FName UText3DCharacterBase::GetRelativeLocationPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeLocation);
}

FName UText3DCharacterBase::GetRelativeRotationPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeRotation);
}

FName UText3DCharacterBase::GetRelativeScalePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeScale);
}

FName UText3DCharacterBase::GetVisiblePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, bVisible);
}

FTransform& UText3DCharacterBase::GetTransform(bool bInReset)
{
	if (bInReset)
	{
		Transform = FTransform(RelativeRotation, RelativeLocation, RelativeScale);
	}

	return Transform;
}

const UE::Text3D::Geometry::FCachedFontFaceGlyphHandle& UText3DCharacterBase::GetFontFaceGlyphHandle() const
{
	return FontFaceGlyphHandle;
}

void UText3DCharacterBase::SetFontFaceGlyphHandle(const UE::Text3D::Geometry::FCachedFontFaceGlyphHandle& InHandle)
{
	FontFaceGlyphHandle = InHandle;
}

void UText3DCharacterBase::SetStyleTag(FName InStyle)
{
	StyleTag = InStyle;
}

uint32 UText3DCharacterBase::GetGlyphIndex() const
{
	return FontFaceGlyphHandle.GetGlyphIndex();
}

const FText3DCachedMesh* UText3DCharacterBase::GetGlyphMesh() const
{
	return FontFaceGlyphHandle.Resolve();
}

void UText3DCharacterBase::GetGlyphMeshBoundsAndOffset(FBox& OutBounds, FVector& OutOffset) const
{
	if (const FText3DCachedMesh* GlyphMesh = GetGlyphMesh())
	{
		OutBounds = GlyphMesh->MeshBounds;
		OutOffset = GlyphMesh->MeshOffset;
	}
	else
	{
		OutBounds = FBox(EForceInit::ForceInitToZero);
		OutOffset = FVector::ZeroVector;
	}
}

void UText3DCharacterBase::SetRelativeLocation(const FVector& InLocation)
{
	if (RelativeLocation.Equals(InLocation))
	{
		return;
	}

	RelativeLocation = InLocation;
	OnCharacterDataChanged(EText3DRendererFlags::Layout);
}

void UText3DCharacterBase::SetRelativeRotation(const FRotator& InRotation)
{
	if (RelativeRotation.Equals(InRotation))
	{
		return;
	}

	RelativeRotation = InRotation;
	OnCharacterDataChanged(EText3DRendererFlags::Layout);
}

void UText3DCharacterBase::SetRelativeScale(const FVector& InScale)
{
	if (RelativeScale.Equals(InScale))
	{
		return;
	}

	RelativeScale = InScale;
	OnCharacterDataChanged(EText3DRendererFlags::Layout);
}

void UText3DCharacterBase::SetVisibility(bool bInVisibility)
{
	if (bVisible == bInVisibility)
	{
		return;
	}

	bVisible = bInVisibility;
	OnCharacterDataChanged(EText3DRendererFlags::Visibility);
}

void UText3DCharacterBase::ResetCharacterState()
{
	const UText3DCharacterBase* CDO = GetDefault<UText3DCharacterBase>();
#if WITH_EDITORONLY_DATA
	Character = CDO->Character;
#endif
	RelativeLocation = CDO->RelativeLocation;
	RelativeRotation = CDO->RelativeRotation;
	RelativeScale = CDO->RelativeScale;
	bVisible = CDO->bVisible;
	Transform = CDO->Transform;
	StyleTag = CDO->StyleTag;
	FontFaceGlyphHandle.Unset();
}

#if WITH_EDITOR
void UText3DCharacterBase::PostEditUndo()
{
	Super::PostEditUndo();

	EText3DRendererFlags Flags = EText3DRendererFlags::Layout;
	EnumAddFlags(Flags, EText3DRendererFlags::Visibility);

	OnCharacterDataChanged(Flags);
}

void UText3DCharacterBase::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> LayoutPropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeLocation),
		GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeRotation),
		GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, RelativeScale)
	};

	const FName MemberPropertyName = InEvent.GetMemberPropertyName();

	if (LayoutPropertyNames.Contains(MemberPropertyName))
	{
		OnCharacterDataChanged(EText3DRendererFlags::Layout);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DCharacterBase, bVisible))
	{
		OnCharacterDataChanged(EText3DRendererFlags::Visibility);
	}
}
#endif

void UText3DCharacterBase::OnCharacterDataChanged(EText3DRendererFlags InFlags) const
{
	if (UText3DComponent* Component = GetTypedOuter<UText3DComponent>())
	{
		Component->RequestUpdate(InFlags);
	}
}
