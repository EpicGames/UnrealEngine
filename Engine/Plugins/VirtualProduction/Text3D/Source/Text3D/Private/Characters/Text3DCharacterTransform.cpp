// Copyright Epic Games, Inc. All Rights Reserved.

#include "Characters/Text3DCharacterTransform.h"
#include "Extensions/Text3DLayoutTransformEffect.h"
#include "GameFramework/Actor.h"
#include "Logs/Text3DLogs.h"
#include "Renderers/StaticMeshes/Text3DStaticMeshesRenderer.h"
#include "Text3DComponent.h"

UText3DCharacterTransform::UText3DCharacterTransform()
{
	bLocationEnabled = false;
	LocationProgress = 0.0f;
	LocationOrder = EText3DCharacterEffectOrder::Normal;
	LocationRange = 50.0f;
	LocationDistance = FVector(100.0f, 0.0f, 0.0f);

	bScaleEnabled = false;
	ScaleProgress = 0.0f;
	ScaleOrder = EText3DCharacterEffectOrder::Normal;
	ScaleRange = 50.0f;
	ScaleBegin = FVector(1.0f, 0.0f, 0.0f);
	ScaleEnd = FVector(1.0f);

	bRotateEnabled = false;
	RotateProgress = 0.0f;
	RotateOrder = EText3DCharacterEffectOrder::Normal;
	RotateRange = 50.0f;
	RotateBegin = FRotator(-90.0f, 0.0f, 0.0f);
	RotateEnd = FRotator(0.0f, 0.0f, 0.0f);
}

void UText3DCharacterTransform::SetLocationEnabled(bool bEnabled)
{
	if (bLocationEnabled != bEnabled)
	{
		bLocationEnabled = bEnabled;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetLocationEnabled(bLocationEnabled);
		}
	}
}

void UText3DCharacterTransform::SetLocationProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(LocationProgress, Progress))
	{
		LocationProgress = Progress;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetLocationProgress(LocationProgress);
		}
	}
}

void UText3DCharacterTransform::SetLocationOrder(EText3DCharacterEffectOrder Order)
{
	if (LocationOrder != Order)
	{
		LocationOrder = Order;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetLocationOrder(LocationOrder);
		}
	}
}

void UText3DCharacterTransform::SetLocationRange(float Range)
{
	Range = FMath::Clamp(Range, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(LocationRange, Range))
	{
		LocationRange = Range;
	}
}

void UText3DCharacterTransform::SetLocationDistance(FVector Distance)
{
	if (!LocationDistance.Equals(Distance))
	{
		LocationDistance = Distance;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetLocationEnd(LocationDistance);
		}
	}
}

void UText3DCharacterTransform::SetScaleEnabled(bool bEnabled)
{
	if (bScaleEnabled != bEnabled)
	{
		bScaleEnabled = bEnabled;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetScaleEnabled(bScaleEnabled);
		}
	}
}

void UText3DCharacterTransform::SetScaleProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(ScaleProgress, Progress))
	{
		ScaleProgress = Progress;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetScaleProgress(ScaleProgress);
		}
	}
}

void UText3DCharacterTransform::SetScaleOrder(EText3DCharacterEffectOrder Order)
{
	if (ScaleOrder != Order)
	{
		ScaleOrder = Order;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetScaleOrder(ScaleOrder);
		}
	}
}

void UText3DCharacterTransform::SetScaleRange(float Range)
{
	Range = FMath::Clamp(Range, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(ScaleRange, Range))
	{
		ScaleRange = Range;
	}
}

void UText3DCharacterTransform::SetScaleBegin(FVector Value)
{
	if (!ScaleBegin.Equals(Value))
	{
		ScaleBegin = Value;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetScaleBegin(ScaleBegin);
		}
	}
}

void UText3DCharacterTransform::SetScaleEnd(FVector Value)
{
	if (!ScaleEnd.Equals(Value))
	{
		ScaleEnd = Value;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetScaleEnd(ScaleEnd);
		}
	}
}

void UText3DCharacterTransform::SetRotateEnabled(bool bEnabled)
{
	if (bRotateEnabled != bEnabled)
	{
		bRotateEnabled = bEnabled;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetRotationEnabled(bRotateEnabled);
		}
	}
}

void UText3DCharacterTransform::SetRotateProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(RotateProgress, Progress))
	{
		RotateProgress = Progress;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetRotationProgress(RotateProgress);
		}
	}
}

void UText3DCharacterTransform::SetRotateOrder(EText3DCharacterEffectOrder Order)
{
	if (RotateOrder != Order)
	{
		RotateOrder = Order;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetRotationOrder(RotateOrder);
		}
	}
}

void UText3DCharacterTransform::SetRotateRange(float Range)
{
	Range = FMath::Clamp(Range, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(RotateRange, Range))
	{
		RotateRange = Range;
	}
}

void UText3DCharacterTransform::SetRotateBegin(FRotator Value)
{
	if (!RotateBegin.Equals(Value))
	{
		RotateBegin = Value;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetRotationBegin(RotateBegin);
		}
	}
}

void UText3DCharacterTransform::SetRotateEnd(FRotator Value)
{
	if (!RotateEnd.Equals(Value))
	{
		RotateEnd = Value;

		if (UText3DLayoutTransformEffect* LayoutTransform = GetText3DLayoutTransformEffect())
		{
			LayoutTransform->SetRotationEnd(RotateEnd);
		}
	}
}

UText3DComponent* UText3DCharacterTransform::GetText3DComponent() const
{
	UText3DComponent* Component = Cast<UText3DComponent>(GetAttachParent());
	if (Component)
	{
		return Component;
	}

	AActor* Owner = GetOwner();
	if (!Owner)
	{
		return nullptr;
	}
	
	return Owner->FindComponentByClass<UText3DComponent>();
}

UText3DLayoutTransformEffect* UText3DCharacterTransform::GetText3DLayoutTransformEffect() const
{
	if (const UText3DComponent* Text3DComponent = GetText3DComponent())
	{
		const TConstArrayView<UText3DLayoutEffectBase*> LayoutEffects = Text3DComponent->GetLayoutEffects();

		if (!LayoutEffects.IsEmpty())
		{
			return Cast<UText3DLayoutTransformEffect>(LayoutEffects[0]);
		}
	}

	return nullptr;
}

void UText3DCharacterTransform::OnComponentCreated()
{
	Super::OnComponentCreated();

	PrintDeprecationLog();
}

void UText3DCharacterTransform::PostLoad()
{
	Super::PostLoad();

	PrintDeprecationLog();
}

void UText3DCharacterTransform::PrintDeprecationLog() const
{
	if (const UText3DComponent* Text3DComponent = GetText3DComponent())
	{
		UE_LOGF(LogText3D, Warning, "%ls : Text3DCharacterTransform component is deprecated and should no longer be used, instead prefer using Text3DLayoutTransformEffect on Text3D that replaces this entirely !", Text3DComponent->GetOwner() ? *Text3DComponent->GetOwner()->GetActorNameOrLabel() : TEXT("Invalid Actor"))
	}
}
