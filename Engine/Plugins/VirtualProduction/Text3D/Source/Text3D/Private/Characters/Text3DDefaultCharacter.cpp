// Copyright Epic Games, Inc. All Rights Reserved.

#include "Characters/Text3DDefaultCharacter.h"

FName UText3DDefaultCharacter::GetKerningPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultCharacter, Kerning);
}

void UText3DDefaultCharacter::SetKerning(float InKerning)
{
	if (FMath::IsNearlyEqual(Kerning, InKerning))
	{
		return;
	}

	Kerning = InKerning;
	OnCharacterDataChanged(EText3DRendererFlags::Layout);
}

#if WITH_EDITOR
void UText3DDefaultCharacter::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	const FName MemberName = InEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UText3DDefaultCharacter, Kerning))
	{
		OnCharacterDataChanged(EText3DRendererFlags::Layout);
	}
}
#endif

float UText3DDefaultCharacter::GetCharacterKerning() const
{
	return Kerning;
}

void UText3DDefaultCharacter::ResetCharacterState()
{
	Super::ResetCharacterState();

	const UText3DDefaultCharacter* CDO = GetDefault<UText3DDefaultCharacter>();
	Kerning = CDO->Kerning;
}
