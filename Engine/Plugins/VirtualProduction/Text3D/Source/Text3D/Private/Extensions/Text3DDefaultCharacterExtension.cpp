// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultCharacterExtension.h"

#include "Characters/Text3DDefaultCharacter.h"
#include "Logs/Text3DLogs.h"
#include "Misc/EnumerateRange.h"
#include "Text3DComponent.h"
#include "Utilities/Text3DUtilities.h"

uint16 UText3DDefaultCharacterExtension::GetCharacterCount() const
{
	return TextCharacters.Num();
}

UText3DCharacterBase* UText3DDefaultCharacterExtension::GetCharacter(uint16 InIndex) const
{
	if (TextCharacters.IsValidIndex(InIndex))
	{
		return TextCharacters[InIndex];
	}

	return nullptr;
}

TConstArrayView<UText3DCharacterBase*> UText3DDefaultCharacterExtension::GetCharacters() const
{
	return TextCharacters;
}

void UText3DDefaultCharacterExtension::AllocateCharacters(uint16 InCount)
{
	AllocateTextCharacters(InCount);
}

void UText3DDefaultCharacterExtension::PostLoad()
{
	Super::PostLoad();

	// Prior to 5.8 TextCharacters were not set to RF_Transactional
	for (UText3DCharacterBase* Character : TextCharacters)
	{
		if (Character)
		{
			Character->SetFlags(RF_Transactional);
		}
	}
}

void UText3DDefaultCharacterExtension::AllocateTextCharacters(uint16 InCharacterCount)
{
	TextCharacters.RemoveAll([](const UText3DCharacterBase* InCharacter)
		{
			return !InCharacter;
		}
	);

	if (TextCharacters.Num() == InCharacterCount)
	{
		return;
	}

	TextCharacters.Reserve(InCharacterCount);

	const int32 RemainingCharacterCount = TextCharacters.Num() - InCharacterCount;

	// Adding to/removing from the TextCharactersPool doesn't set/clear the transient flags
	// so needs to be done manually here
	constexpr EObjectFlags PoolFlags = RF_Transient | RF_DuplicateTransient | RF_TextExportTransient;

	if (RemainingCharacterCount > 0)
	{
		for (int32 CharacterIndex = InCharacterCount; CharacterIndex < TextCharacters.Num(); CharacterIndex++)
		{
			if (UText3DCharacterBase* Character = TextCharacters[CharacterIndex])
			{
				Character->ResetCharacterState();
				Character->SetFlags(PoolFlags);
				TextCharactersPool.Add(Character);
			}
		}
	}
	else if (RemainingCharacterCount < 0)
	{
		for (int32 CharacterIndex = 0; CharacterIndex < FMath::Abs(RemainingCharacterCount); ++CharacterIndex)
		{
			UText3DCharacterBase* TextCharacter = nullptr;

			while (!TextCharactersPool.IsEmpty() && !TextCharacter)
			{
				TextCharacter = TextCharactersPool.Pop();
			}

			if (!TextCharacter)
			{
				const FName ObjectName = MakeUniqueObjectName(this, UText3DDefaultCharacter::StaticClass(), FName(TEXT("Char")));
				TextCharacter = NewObject<UText3DCharacterBase>(this, UText3DDefaultCharacter::StaticClass(), ObjectName, RF_Transactional);
			}

			TextCharacter->ClearFlags(PoolFlags);
			TextCharacters.Add(TextCharacter);
		}
	}

	TextCharacters.SetNum(InCharacterCount);
}
