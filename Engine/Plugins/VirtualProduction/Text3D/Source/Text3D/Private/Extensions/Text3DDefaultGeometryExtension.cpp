// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultGeometryExtension.h"

#include "Characters/Text3DCharacterBase.h"
#include "Engine/Font.h"
#include "Engine/StaticMesh.h"
#include "Logs/Text3DLogs.h"
#include "MeshDescription.h"
#include "Renderers/Text3DRendererBase.h"
#include "Settings/Text3DProjectSettings.h"
#include "Subsystems/Text3DEngineSubsystem.h"
#include "Text3DComponent.h"

void UText3DDefaultGeometryExtension::SetUseOutline(const bool bValue)
{
	if (bUseOutline == bValue)
	{
		return;
	}

	bUseOutline = bValue;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetOutline(const float Value)
{
	if (FMath::IsNearlyEqual(Outline, Value))
	{
		return;
	}

	Outline = Value;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetOutlineType(EText3DOutlineType InType)
{
	if (OutlineType == InType)
	{
		return;
	}

	OutlineType = InType;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetPivotHAlignment(EText3DHorizontalTextAlignment InPivot)
{
	if (PivotHAlignment == InPivot)
	{
		return;
	}

	PivotHAlignment = InPivot;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetPivotVAlignment(EText3DVerticalTextAlignment InPivot)
{
	if (PivotVAlignment == InPivot)
	{
		return;
	}

	// Only one supported for now
	if (InPivot != EText3DVerticalTextAlignment::Bottom)
	{
		return;
	}

	PivotVAlignment = InPivot;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetCollisionEnabled(ECollisionEnabled::Type InCollisionEnabled)
{
	if (CollisionEnabled == InCollisionEnabled)
	{
		return;
	}

	CollisionEnabled = InCollisionEnabled;
	OnCollisionEnabledChanged();
}

void UText3DDefaultGeometryExtension::SetExtrude(const float Value)
{
	const float NewValue = FMath::Max(0.0f, Value);
	if (FMath::IsNearlyEqual(Extrude, NewValue))
	{
		return;
	}

	Extrude = NewValue;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetBevel(const float Value)
{
	const float NewValue = FMath::Clamp(Value, 0.f, GetMaxBevel());
	if (FMath::IsNearlyEqual(Bevel, NewValue))
	{
		return;
	}
	
	Bevel = NewValue;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetBevelType(const EText3DBevelType Value)
{
	if (BevelType == Value)
	{
		return;
	}

	BevelType = Value;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::SetBevelSegments(const int32 Value)
{
	int32 MinBevelSegments = 1;
	if (BevelType == EText3DBevelType::HalfCircle)
	{
		MinBevelSegments = 2;
	}

	const int32 NewValue = FMath::Clamp(Value, MinBevelSegments, 15);
	if (BevelSegments == NewValue)
	{
		return;
	}
	
	BevelSegments = NewValue;
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const UText3DProjectSettings* const Settings = UText3DProjectSettings::Get();
		if (Settings && Settings->ShouldForceDisableTextCollision())
		{
			CollisionEnabled = ECollisionEnabled::NoCollision;
		}
	}
}

#if WITH_EDITOR
void UText3DDefaultGeometryExtension::PostEditUndo()
{
	Super::PostEditUndo();
	OnGeometryOptionsChanged();
}

void UText3DDefaultGeometryExtension::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, Extrude),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, Bevel),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, BevelType),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, BevelSegments),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, bUseOutline),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, Outline),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, OutlineType),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, PivotHAlignment),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, PivotVAlignment),
	};

	if (PropertyNames.Contains(InEvent.GetMemberPropertyName()))
	{
		OnGeometryOptionsChanged();
	}
	else if (InEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UText3DDefaultGeometryExtension, CollisionEnabled))
	{
		OnCollisionEnabledChanged();
	}
}
#endif

EText3DHorizontalTextAlignment UText3DDefaultGeometryExtension::GetGlyphHAlignment() const
{
	return PivotHAlignment;
}

EText3DVerticalTextAlignment UText3DDefaultGeometryExtension::GetGlyphVAlignment() const
{
	return PivotVAlignment;
}

ECollisionEnabled::Type UText3DDefaultGeometryExtension::GetGlyphCollisionEnabled() const
{
	return CollisionEnabled;
}

const FGlyphMeshParameters* UText3DDefaultGeometryExtension::GetGlyphMeshParameters() const
{
	return &GlyphMeshParameters;
}

void UText3DDefaultGeometryExtension::PrepareBuild(EText3DRendererFlags InUpdateFlags)
{
	if (!EnumHasAnyFlags(InUpdateFlags, EText3DRendererFlags::Geometry))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultGeometryExtension::PrepareRebuild);

	// Refresh typeface to ensure type face index is up-to-date.
	UText3DComponent* const TextComponent = GetText3DComponent();
	TextComponent->RefreshTypeface();

	const int32 TypefaceIndex = TextComponent->GetTypeFaceIndex();
	if (TypefaceIndex == INDEX_NONE)
	{
		const UFont* Font = TextComponent->GetFont();
		UE_LOGF(LogText3D, Error, "Invalid typeface index returned for font %ls : %i", Font ? *Font->GetName() : TEXT("invalid"), TypefaceIndex);
		return;
	}

	const float GlyphExtrude = FMath::TruncToFloat(GetExtrude() * 10.f) / 10.f;
	const float GlyphBevel = FMath::TruncToFloat(GetBevel() * 10.f) / 10.f;
	const float GlyphOutline = FMath::TruncToFloat(GetOutline() * 10.f) / 10.f;

	const UText3DRendererBase* TextRenderer = TextComponent->GetTextRendererClass().GetDefaultObject();
	GlyphMeshParameters =
	{
		.Extrude = GlyphExtrude,
		.Bevel = GlyphBevel,
		.BevelType = GetBevelType(),
		.BevelSegments = GetBevelSegments(),
		.bOutline = GetUseOutline(),
		.OutlineExpand = GlyphOutline,
		.OutlineType = GetOutlineType(),
		.PivotOffset = GetPivotOffset(),
		.MeshType = TextRenderer ? TextRenderer->GetMeshType() : EText3DMeshType::Unknown,
	};
}

EText3DExtensionResult UText3DDefaultGeometryExtension::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	// Keep to clean after
	return EText3DExtensionResult::Active;
}

EText3DExtensionResult UText3DDefaultGeometryExtension::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Finished;
}

void UText3DDefaultGeometryExtension::OnGeometryOptionsChanged()
{
	// Extrude
	Extrude = FMath::Max(0.0f, Extrude);

	// Bevels
	Bevel = FMath::Clamp(Bevel, 0.f, GetMaxBevel());

	int32 MaxBevelSegments = 1;
	int32 MinBevelSegments = 1;
	switch (BevelType)
	{
	case EText3DBevelType::Linear:
	case EText3DBevelType::OneStep:
	case EText3DBevelType::TwoSteps:
	case EText3DBevelType::Engraved:
		{
			MaxBevelSegments = 1;
			break;
		}
	case EText3DBevelType::Convex:
	case EText3DBevelType::Concave:
		{
			MaxBevelSegments = 8;
			break;
		}
	case EText3DBevelType::HalfCircle:
		{
			MaxBevelSegments = 16;
			MinBevelSegments = 2;
			break;
		}
	}

	BevelSegments = FMath::Clamp(BevelSegments, MinBevelSegments, MaxBevelSegments);
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DDefaultGeometryExtension::OnCollisionEnabledChanged()
{
	RequestUpdate(EText3DRendererFlags::Geometry);
}

float UText3DDefaultGeometryExtension::GetMaxBevel() const
{
	return Extrude / 2.0f;
}

FVector UText3DDefaultGeometryExtension::GetPivotOffset() const
{
	switch (PivotHAlignment)
	{
	case EText3DHorizontalTextAlignment::Left:
		return FVector::ZeroVector;
	case EText3DHorizontalTextAlignment::Center:
		return FVector(0.0f, 0.5f, 0.0f);
	case EText3DHorizontalTextAlignment::Right:
		return FVector(0.0f, 1.f, 0.0f);
	}

	return FVector::ZeroVector;
}
