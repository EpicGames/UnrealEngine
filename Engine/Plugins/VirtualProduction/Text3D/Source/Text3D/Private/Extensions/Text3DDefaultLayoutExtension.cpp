// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultLayoutExtension.h"

#include "Algo/Accumulate.h"
#include "Algo/Count.h"
#include "Async/ParallelFor.h"
#include "Characters/Text3DCharacterBase.h"
#include "Engine/Font.h"
#include "Extensions/Text3DCharacterExtensionBase.h"
#include "Extensions/Text3DGeometryExtensionBase.h"
#include "Extensions/Text3DStyleExtensionBase.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Framework/Text/RichTextMarkupProcessing.h"
#include "GeometryBuilders/Text3DGlyphOutline.h"
#include "LayoutBuilders/Text3DLayout.h"
#include "LayoutBuilders/Text3DLayoutShaper.h"
#include "LayoutBuilders/Text3DShapedGlyphText.h"
#include "Logs/Text3DLogs.h"
#include "Materials/Material.h"
#include "Misc/EnumerateRange.h"
#include "Settings/Text3DProjectSettings.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Subsystems/Text3DEngineSubsystem.h"
#include "Text3DComponent.h"

#if WITH_FREETYPE
#include "Fonts/SlateTextShaper.h"
#endif

FName UText3DDefaultLayoutExtension::GetUseMaxWidthPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bUseMaxWidth);
}

FName UText3DDefaultLayoutExtension::GetUseMaxHeightPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bUseMaxHeight);
}

FName UText3DDefaultLayoutExtension::GetMaxHeightPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, MaxHeight);
}

FName UText3DDefaultLayoutExtension::GetMaxWidthPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, MaxWidth);
}

FName UText3DDefaultLayoutExtension::GetScaleProportionallyPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bScaleProportionally);
}

void UText3DDefaultLayoutExtension::SetTracking(const float Value)
{
	if (FMath::IsNearlyEqual(Tracking, Value))
	{
		return;
	}
	
	Tracking = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetLineSpacing(const float Value)
{
	if (FMath::IsNearlyEqual(LineSpacing, Value))
	{
		return;
	}
	
	LineSpacing = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetWordSpacing(const float Value)
{
	if (FMath::IsNearlyEqual(WordSpacing, Value))
	{
		return;
	}

	WordSpacing = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetHorizontalAlignment(const EText3DHorizontalTextAlignment Value)
{
	if (HorizontalAlignment == Value)
	{
		return;
	}

	HorizontalAlignment = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetVerticalAlignment(const EText3DVerticalTextAlignment Value)
{
	if (VerticalAlignment == Value)
	{
		return;
	}
	
	VerticalAlignment = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetUseMaxWidth(const bool Value)
{
	if (bUseMaxWidth == Value)
	{
		return;
	}

	bUseMaxWidth = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetMaxWidth(const float Value)
{
	const float NewValue = FMath::Max(1.0f, Value);
	if (FMath::IsNearlyEqual(MaxWidth, NewValue))
	{
		return;
	}
	
	MaxWidth = NewValue;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetMaxWidthBehavior(const EText3DMaxWidthHandling Value)
{
	if (MaxWidthBehavior == Value)
	{
		return;
	}

	MaxWidthBehavior = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetUseMaxHeight(const bool Value)
{
	if (bUseMaxHeight == Value)
	{
		return;
	}
	
	bUseMaxHeight = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetMaxHeight(const float Value)
{
	const float NewValue = FMath::Max(1.0f, Value);
	if (FMath::IsNearlyEqual(MaxHeight, NewValue))
	{
		return;
	}

	MaxHeight = NewValue;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetScaleProportionally(const bool Value)
{
	if (bScaleProportionally == Value)
	{
		return;
	}

	bScaleProportionally = Value;
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::SetTextShapingMethod(ETextShapingMethod InTextShapingMethod)
{
	if (TextShapingMethod == InTextShapingMethod)
	{
		return;
	}

	TextShapingMethod = InTextShapingMethod;
	OnTextShapingMethodChanged();
}

float UText3DDefaultLayoutExtension::GetTextHeight() const
{
	using namespace UE::Text3D::Layout;

	float Height = FMath::Max(GlyphText->Lines.Num() - 1, 0) * LineSpacing;

	for (const FGlyphLine& Line : GlyphText->Lines)
	{
		Height += Line.MaxFontHeight;
	}

	return Height;
}

FVector UText3DDefaultLayoutExtension::GetTextScale() const
{
	return TextScale;
}

void UText3DDefaultLayoutExtension::PrepareBuild(EText3DRendererFlags InUpdateFlags)
{
	// Currently only concerned about geometry / glyph building
	if (!EnumHasAnyFlags(InUpdateFlags, EText3DRendererFlags::Geometry))
	{
		return;
	}

	GlyphHandles.Empty();

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultLayoutExtension::PrepareRebuild);

	using namespace UE::Text3D::Layout;

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	
	if (!GlyphText.IsValid())
	{
		GlyphText = MakeShared<FGlyphText>();
	}

	if (!TextLayout.IsValid())
	{
		TextLayout = MakeShared<FText3DLayout>();
	}
	TextLayout->SetTextShapingMethod(TextShapingMethod);

	if (!TextLayoutMarshaller.IsValid())
	{
		const TSharedPtr<IRichTextMarkupParser> Parser = FDefaultRichTextMarkupParser::GetStaticInstance();
		TextLayoutMarshaller = FRichTextLayoutMarshaller::Create(Parser, nullptr, TArray<TSharedRef<ITextDecorator>>(), nullptr);
	}

	GlyphText->Reset();

	const UText3DStyleExtensionBase* FormatStyleExtension = Text3DComponent->GetStyleExtension();
	FText3DLayoutShaper::Get()->ShapeBidirectionalText(FormatStyleExtension->GetCustomStyles(), *FormatStyleExtension->GetDefaultStyle(), Text3DComponent->GetFormattedText().ToString(), TextLayout, TextLayoutMarshaller, GlyphText->Lines);

#if WITH_FREETYPE
	UText3DEngineSubsystem* const TextSubsystem = UText3DEngineSubsystem::Get();
	if (!ensure(TextSubsystem))
	{
		return;
	}

	const UText3DGeometryExtensionBase* const GeometryExtension = Text3DComponent->GetGeometryExtension();
	if (!ensure(GeometryExtension))
	{
		return;
	}

	const FGlyphMeshParameters& GlyphMeshParameters = *GeometryExtension->GetGlyphMeshParameters();
	TArray<FText3DBuildGlyphMeshDesc> BuildGlyphMeshDescs;

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultLayoutExtension::PrepareCachedMeshes);
	for (const TConstEnumerateRef<FGlyphLine> GlyphLine : EnumerateRange(GlyphText->Lines))
	{
		for (const TConstEnumerateRef<FGlyphEntry> GlyphEntry : EnumerateRange(GlyphLine->Glyphs))
		{
			if (!FGlyphText::IsValidGlyph(*GlyphEntry))
			{
				continue;
			}
			const TSharedPtr<FFreeTypeFace> FontFace = GlyphEntry->Entry.FontFaceData->FontFace.Pin();
			if (FText3DFontFaceCache* FontFaceCache = TextSubsystem->FindOrAddCachedFontFace(FontFace))
			{
				bool bAlreadyCached = false;
				TOptional<FText3DBuildGlyphMeshDesc> GlyphOutline = FontFaceCache->PrepareCachedMesh(GlyphEntry->Entry.GlyphIndex, GlyphMeshParameters, /*Out*/bAlreadyCached);
				if (GlyphOutline.IsSet() || bAlreadyCached)
				{
					// Keep handles of glyph meshes to prevent these from being cleaned up. Cache cleanup can run in between here and the next stage (for time-sliced builds).
					GlyphHandles.Emplace(*FontFaceCache, GlyphEntry->Entry.GlyphIndex, GlyphMeshParameters);
				}
				if (GlyphOutline.IsSet())
				{
					BuildGlyphMeshDescs.Emplace(MoveTemp(*GlyphOutline));
				}
			}
		}
	}

	TextSubsystem->QueueBuildGlyphMeshes(MoveTemp(BuildGlyphMeshDescs), GlyphMeshParameters);
	BuildGlyphMeshDescs.Reset();
#else
	UE_LOGF(LogText3D, Warning, "FreeType is not available, cannot proceed without it");
#endif
}

EText3DExtensionResult UText3DDefaultLayoutExtension::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	using namespace UE::Text3D::Layout;

	if (InParameters.CurrentFlag != EText3DRendererFlags::Geometry && InParameters.CurrentFlag != EText3DRendererFlags::Layout)
	{
		return EText3DExtensionResult::Active;
	}

	if (!GlyphText.IsValid())
	{
		UE_LOGF(LogText3D, Warning, "Text3DDefaultLayoutExtension::PreRendererUpdate failed. Invalid GlyphText");
		return EText3DExtensionResult::Failed;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultLayoutExtension::PreRendererUpdate);

	const UText3DComponent* Text3DComponent = GetText3DComponent();

	const int32 CharacterCount = Algo::TransformAccumulate(GlyphText->Lines, [](const FGlyphLine& InLine)
	{
		return Algo::CountIf(InLine.Glyphs, [](const FGlyphEntry& InGlyph)
		{
			return FGlyphText::IsValidGlyph(InGlyph);
		});
	},0);

	if (EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Geometry))
	{
		if (UText3DCharacterExtensionBase* CharacterExtension = Text3DComponent->GetCharacterExtension())
		{
			CharacterExtension->AllocateCharacters(CharacterCount);
		}
		else
		{
			UE_LOGF(LogText3D, Warning, "Text3D character extension is invalid, cannot proceed without it")
			return EText3DExtensionResult::Failed;
		}
	}

	GlyphText->Kernings.SetNum(CharacterCount);
	GlyphText->FontFaces.SetNum(CharacterCount);

	int32 CharacterIndex = 0;

#if WITH_FREETYPE
	for (const TConstEnumerateRef<FGlyphLine> GlyphLine : EnumerateRange(GlyphText->Lines))
	{
		for (const TConstEnumerateRef<FGlyphEntry> GlyphEntry : EnumerateRange(GlyphLine->Glyphs))
		{
			if (FGlyphText::IsValidGlyph(*GlyphEntry))
			{
				const TSharedPtr<FFreeTypeFace> FontFacePtr = GlyphEntry->Entry.FontFaceData->FontFace.Pin();
				if (const UText3DCharacterBase* Character = Text3DComponent->GetCharacter(CharacterIndex))
				{
					GlyphText->Kernings[CharacterIndex] = Character->GetCharacterKerning();
                    GlyphText->FontFaces[CharacterIndex] = FontFacePtr;
                    CharacterIndex++;
				}
				else
				{
					UE_LOGF(LogText3D, Warning, "Invalid character object returned for index %i before layout calculations, cannot proceed", CharacterIndex)
					return EText3DExtensionResult::Failed;
				}
			}
		}
	}

	GlyphText->Tracking = Tracking;
	GlyphText->WordSpacing = WordSpacing;
	GlyphText->MaxWidth = MaxWidth;
	GlyphText->bWrap = bUseMaxWidth && MaxWidthBehavior == EText3DMaxWidthHandling::WrapAndScale;
	GlyphText->CalculateWidth();
	CalculateTextScale();

	UText3DEngineSubsystem* const TextSubsystem = UText3DEngineSubsystem::Get();
	if (!ensure(TextSubsystem))
	{
		return EText3DExtensionResult::Failed;
	}

	const UText3DGeometryExtensionBase* const GeometryExtension = Text3DComponent->GetGeometryExtension();
	if (!ensure(GeometryExtension))
	{
		return EText3DExtensionResult::Failed;
	}

	const FGlyphMeshParameters& GlyphMeshParameters = *GeometryExtension->GetGlyphMeshParameters();

	if (EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Geometry))
	{
		// Call build glyph meshes for blocking builds (Incremental builds already call this in the build system)
		TextSubsystem->ProcessBuildGlyphMeshes();	
	}

	CharacterIndex = 0;

	for (TConstEnumerateRef<FGlyphLine> GlyphLine : EnumerateRange(GlyphText->Lines))
	{
		FVector Location = GetLineLocation(GlyphLine.GetIndex());
		for (TConstEnumerateRef<FGlyphEntry> GlyphEntry : EnumerateRange(GlyphLine->Glyphs))
		{
			if (FGlyphText::IsValidGlyph(*GlyphEntry))
			{
				UText3DCharacterBase* Character = Text3DComponent->GetCharacter(CharacterIndex);

				if (!Character)
				{
					UE_LOGF(LogText3D, Warning, "Invalid character object returned for index %i after layout calculations, cannot proceed", CharacterIndex)
					return EText3DExtensionResult::Failed;
				}

				if (EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Geometry))
				{
					TSharedPtr<FFreeTypeFace> FontFace = GlyphText->FontFaces[CharacterIndex];
					// Cache font face to keep it alive and use inner FT_Face to build glyphs
					FText3DFontFaceCache* FontFaceCache = TextSubsystem->FindOrAddCachedFontFace(FontFace);

					if (FontFaceCache)
					{
						UE::Text3D::Geometry::FCachedFontFaceGlyphHandle GlyphHandle(*FontFaceCache, GlyphEntry->Entry.GlyphIndex, GlyphMeshParameters);
						Character->SetFontFaceGlyphHandle(GlyphHandle);
						Character->SetStyleTag(GlyphEntry->StyleTag);
					}
					else
					{
						UE_LOGF(LogText3D, Warning, "Failed to get cached font data for '%u %i' in Text3D geometry extension", FText3DFontFaceCache::GetFontFaceHash(FontFace), GlyphEntry->Entry.GlyphIndex);
						return EText3DExtensionResult::Failed;
					}
				}

				const FText3DCachedMesh* CachedMesh = Character->GetGlyphMesh();

				if (!CachedMesh)
				{
					UE_LOGF(LogText3D, Warning, "Invalid character cached mesh returned for index %i and glyph %i, cannot proceed", CharacterIndex, GlyphEntry->Entry.GlyphIndex)
					return EText3DExtensionResult::Failed;
				}

				const FVector GlyphSize = CachedMesh->MeshBounds.GetSize();

				const FVector2D FaceSize = FVector2D::Max(CachedMesh->FontFaceGlyphSize, FVector2D(UE_SMALL_NUMBER));
				const FVector Ratio(1.f, GlyphEntry->Size / FaceSize.X, GlyphEntry->Size / FaceSize.Y);

				FVector Shift = FVector::ZeroVector;
				switch (GeometryExtension->GetGlyphHAlignment())
				{
				case EText3DHorizontalTextAlignment::Center:
					Shift.Y += (0.5 * GlyphSize.Y + CachedMesh->MeshOffset.Y) * Ratio.Y;
					break;
				case EText3DHorizontalTextAlignment::Right:
					Shift.Y += (GlyphSize.Y + CachedMesh->MeshOffset.Y) * Ratio.Y;
					break;
				}

				if (GlyphEntry.GetIndex() != 0)
				{
					Shift.Y += Tracking + GlyphText->Kernings[CharacterIndex];
				}

				// Handle special characters with no advance but offset
				if (GlyphEntry->Entry.XAdvance == 0)
				{
					Shift.Y += GlyphEntry->Entry.XOffset;
					Shift.Z -= GlyphEntry->Entry.YOffset;
				}

				FTransform Transform(Location + Shift);
				Transform.SetScale3D(Ratio);

				constexpr bool bReset = true;
				FTransform& CharacterTransform = Character->GetTransform(bReset);
				CharacterTransform.Accumulate(Transform);
				CharacterIndex++;
			}

			Location.Y += GlyphLine->GlyphAdvances[GlyphEntry.GetIndex()];
		}
	}

	if (EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Geometry))
	{
		// Glyph meshes have been processed. No longer need to keep handle to these.
		GlyphHandles.Empty();
	}
	return EText3DExtensionResult::Finished;
#else
	UE_LOGF(LogText3D, Warning, "FreeType is not available, cannot proceed without it")
	return EText3DExtensionResult::Failed;
#endif // WITH_FREETYPE
}

EText3DExtensionResult UText3DDefaultLayoutExtension::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Active;
}

void UText3DDefaultLayoutExtension::CalculateTextScale()
{
	using namespace UE::Text3D::Layout;

	FVector Scale(1.0f, 1.0f, 1.0f);

	float TextMaxWidth = 0.0f;
	for (const FGlyphLine& ShapedLine : GlyphText->Lines)
	{
		TextMaxWidth = FMath::Max(TextMaxWidth, ShapedLine.Width);
	}

	if (bUseMaxWidth && TextMaxWidth > MaxWidth && TextMaxWidth > 0.0f)
	{
		Scale.Y *= MaxWidth / TextMaxWidth;
		if (bScaleProportionally)
		{
			Scale.Z = Scale.Y;
		}
	}

	const float TotalHeight = GetTextHeight();
	if (bUseMaxHeight && TotalHeight > MaxHeight && TotalHeight > 0.0f)
	{
		Scale.Z *= MaxHeight / TotalHeight;
		if (bScaleProportionally)
		{
			Scale.Y = Scale.Z;
		}
	}

	if (bScaleProportionally)
	{
		Scale.X = Scale.Y;
	}

	TextScale = Scale;
}

FVector UText3DDefaultLayoutExtension::GetLineLocation(int32 LineIndex) const
{
	using namespace UE::Text3D::Layout;

	float HorizontalOffset = 0.0f;
	float VerticalOffset = 0.0f;

	if (LineIndex < 0 || LineIndex >= GlyphText->Lines.Num())
	{
		return FVector::ZeroVector;
	}

	const FGlyphLine& ShapedLine = GlyphText->Lines[LineIndex];

	if (HorizontalAlignment == EText3DHorizontalTextAlignment::Center)
	{
		HorizontalOffset = -ShapedLine.Width * 0.5f;
	}
	else if (HorizontalAlignment == EText3DHorizontalTextAlignment::Right)
	{
		HorizontalOffset = -ShapedLine.Width;
	}

	const float TotalHeight = GetTextHeight();
	if (VerticalAlignment != EText3DVerticalTextAlignment::FirstLine)
	{
		// First align it to Top
		VerticalOffset -= ShapedLine.MaxFontAscender;

		if (VerticalAlignment == EText3DVerticalTextAlignment::Center)
		{
			VerticalOffset += TotalHeight * 0.5f;
		}
		else if (VerticalAlignment == EText3DVerticalTextAlignment::Bottom)
		{
			VerticalOffset += TotalHeight - ShapedLine.MaxFontDescender;
		}
	}

	// Offset line based on cumulative height above it
	for (int32 Index = 0; Index < LineIndex; ++Index)
	{
		const FGlyphLine& PrevLine = GlyphText->Lines[Index];
		VerticalOffset -= (PrevLine.MaxFontHeight + LineSpacing);
	}

	return FVector(0.0f, HorizontalOffset, VerticalOffset);
}

#if WITH_EDITOR
void UText3DDefaultLayoutExtension::PostEditUndo()
{
	Super::PostEditUndo();
	OnLayoutOptionsChanged();
}

void UText3DDefaultLayoutExtension::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> LayoutPropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, Tracking),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, LineSpacing),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, WordSpacing),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, HorizontalAlignment),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, VerticalAlignment),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bUseMaxWidth),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, MaxWidth),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bUseMaxHeight),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, MaxHeight),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, bScaleProportionally),
	};

	const FName ChangedPropertyName = InEvent.GetMemberPropertyName();
	if (LayoutPropertyNames.Contains(ChangedPropertyName))
	{
		OnLayoutOptionsChanged();
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UText3DDefaultLayoutExtension, TextShapingMethod))
	{
		OnTextShapingMethodChanged();
	}
}
#endif

void UText3DDefaultLayoutExtension::OnLayoutOptionsChanged()
{
	using namespace UE::Text3D::Layout;

	MaxWidth = FMath::Max(1.0f, MaxWidth);
	MaxHeight = FMath::Max(1.0f, MaxHeight);

	EText3DRendererFlags Flags = EText3DRendererFlags::Material | EText3DRendererFlags::Layout;

	if (MaxWidthBehavior == EText3DMaxWidthHandling::WrapAndScale && GlyphText.IsValid())
	{
		for (const TConstEnumerateRef<FGlyphLine> ShapedLine : EnumerateRange(GlyphText->Lines))
		{
			if (ShapedLine->TextDirection == TextBiDi::ETextDirection::RightToLeft)
			{
				Flags |= EText3DRendererFlags::Geometry;
				break;
			}
		}
	}

	RequestUpdate(Flags);
}

void UText3DDefaultLayoutExtension::OnTextShapingMethodChanged()
{
	// Changing text shaping method requires recomputing geometry too
	constexpr EText3DRendererFlags Flags = EText3DRendererFlags::Material | EText3DRendererFlags::Layout | EText3DRendererFlags::Geometry;
	RequestUpdate(Flags);
}
