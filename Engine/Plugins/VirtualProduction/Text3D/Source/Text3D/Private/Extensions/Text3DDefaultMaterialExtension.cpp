// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultMaterialExtension.h"

#include "Engine/Texture2D.h"
#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Extensions/Text3DStyleExtensionBase.h"
#include "Logs/Text3DLogs.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/EnumerateRange.h"
#include "Settings/Text3DProjectSettings.h"
#include "Styles/Text3DStyleBase.h"
#include "Text3DComponent.h"
#include "UObject/Package.h"

void UText3DDefaultMaterialExtension::SetStyle(EText3DMaterialStyle InStyle)
{
	if (Style == InStyle)
	{
		return;
	}

	Style = InStyle;
	OnCustomMaterialChanged();
}

void UText3DDefaultMaterialExtension::SetFrontColor(const FLinearColor& InColor)
{
	if (FrontColor.Equals(InColor))
	{
		return;
	}

	FrontColor = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetBackColor(const FLinearColor& InColor)
{
	if (BackColor.Equals(InColor))
	{
		return;
	}

	BackColor = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetExtrudeColor(const FLinearColor& InColor)
{
	if (ExtrudeColor.Equals(InColor))
	{
		return;
	}

	ExtrudeColor = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetBevelColor(const FLinearColor& InColor)
{
	if (BevelColor.Equals(InColor))
	{
		return;
	}

	BevelColor = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetGradientColorA(const FLinearColor& InColor)
{
	if (GradientColorA.Equals(InColor))
	{
		return;
	}
	
	GradientColorA = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetGradientColorB(const FLinearColor& InColor)
{
	if (GradientColorB.Equals(InColor))
	{
		return;
	}

	GradientColorB = InColor;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetGradientSmoothness(float InGradientSmoothness)
{
	if (FMath::IsNearlyEqual(GradientSmoothness, InGradientSmoothness))
	{
		return;
	}

	GradientSmoothness = InGradientSmoothness;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetGradientOffset(float InGradientOffset)
{
	if (FMath::IsNearlyEqual(GradientOffset, InGradientOffset))
	{
		return;
	}

	GradientOffset = InGradientOffset;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetGradientRotation(float InGradientRotation)
{
	if (FMath::IsNearlyEqual(GradientRotation, InGradientRotation))
	{
		return;
	}

	GradientRotation = InGradientRotation;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetTextureAsset(UTexture2D* InTextureAsset)
{
	if (TextureAsset == InTextureAsset)
	{
		return;
	}

	TextureAsset = InTextureAsset;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetTextureTiling(const FVector2D& InTextureTiling)
{
	if (TextureTiling.Equals(InTextureTiling))
	{
		return;
	}

	TextureTiling = InTextureTiling;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetBlendMode(EText3DMaterialBlendMode InBlendMode)
{
	if (BlendMode == InBlendMode)
	{
		return;
	}

	BlendMode = InBlendMode;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetIsUnlit(bool bInIsUnlit)
{
	if (bIsUnlit == bInIsUnlit)
	{
		return;
	}

	bIsUnlit = bInIsUnlit;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetOpacity(float InOpacity)
{
	if (FMath::IsNearlyEqual(Opacity, InOpacity))
	{
		return;
	}

	Opacity = InOpacity;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetUseMask(bool bInUseMask)
{
	if (bUseMask == bInUseMask)
	{
		return;
	}

	bUseMask = bInUseMask;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetMaskOffset(float InMaskOffset)
{
	if (FMath::IsNearlyEqual(MaskOffset, InMaskOffset))
	{
		return;
	}

	MaskOffset = InMaskOffset;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetMaskSmoothness(float InMaskSmoothness)
{
	if (FMath::IsNearlyEqual(MaskSmoothness, InMaskSmoothness))
	{
		return;
	}

	MaskSmoothness = InMaskSmoothness;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetMaskRotation(float InMaskRotation)
{
	if (FMath::IsNearlyEqual(MaskRotation, InMaskRotation))
	{
		return;
	}

	MaskRotation = InMaskRotation;
	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::SetUseSingleMaterial(bool bInUse)
{
	if (bUseSingleMaterial == bInUse)
	{
		return;
	}

	bUseSingleMaterial = bInUse;
	OnCustomMaterialChanged();
}

void UText3DDefaultMaterialExtension::SetFrontMaterial(UMaterialInterface* InMaterial)
{
	if (FrontMaterial == InMaterial)
	{
		return;
	}

	FrontMaterial = InMaterial;
	OnCustomMaterialChanged();
}

void UText3DDefaultMaterialExtension::SetBevelMaterial(UMaterialInterface* InMaterial)
{
	if (BevelMaterial == InMaterial)
	{
		return;
	}
	
	BevelMaterial = InMaterial;
	OnCustomMaterialChanged();
}

void UText3DDefaultMaterialExtension::SetExtrudeMaterial(UMaterialInterface* InMaterial)
{
	if (ExtrudeMaterial == InMaterial)
	{
		return;
	}
	
	ExtrudeMaterial	= InMaterial;
	OnCustomMaterialChanged();
}

void UText3DDefaultMaterialExtension::SetBackMaterial(UMaterialInterface* InMaterial)
{
	if (BackMaterial == InMaterial)
	{
		return;
	}
	
	BackMaterial = InMaterial;
	OnCustomMaterialChanged();
}

EText3DExtensionResult UText3DDefaultMaterialExtension::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	if (InParameters.CurrentFlag != EText3DRendererFlags::Material)
	{
		return EText3DExtensionResult::Active;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();

	if (Style != EText3DMaterialStyle::Custom && Style != EText3DMaterialStyle::Invalid)
	{
		UMaterialInstanceDynamic* DynFrontMaterial = FindOrAdd(EText3DGroupType::Front);
		UMaterialInstanceDynamic* DynBackMaterial = FindOrAdd(EText3DGroupType::Back);
		UMaterialInstanceDynamic* DynExtrudeMaterial = FindOrAdd(EText3DGroupType::Extrude);
		UMaterialInstanceDynamic* DynBevelMaterial = FindOrAdd(EText3DGroupType::Bevel);

		if (!DynFrontMaterial || !DynBackMaterial || !DynExtrudeMaterial || !DynBevelMaterial)
		{
			UE_LOGF(LogText3D, Error, "Failed to retrieve dynamic material in Text3D material extension")
			return EText3DExtensionResult::Failed;
		}

		TArray<UMaterialInstanceDynamic*> Materials
		{
			DynFrontMaterial,
			DynBackMaterial,
			DynExtrudeMaterial,
			DynBevelMaterial
		};

		switch (Style)
		{
		default:	
		case EText3DMaterialStyle::Solid:
			DynFrontMaterial->SetVectorParameterValue(UText3DProjectSettings::FMaterialParameters::SolidColor, FrontColor);
			DynBackMaterial->SetVectorParameterValue(UText3DProjectSettings::FMaterialParameters::SolidColor, BackColor);
			DynExtrudeMaterial->SetVectorParameterValue(UText3DProjectSettings::FMaterialParameters::SolidColor, ExtrudeColor);
			DynBevelMaterial->SetVectorParameterValue(UText3DProjectSettings::FMaterialParameters::SolidColor, BevelColor);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::Mode, 0);
			break;
		case EText3DMaterialStyle::Gradient:
			SetVectorParameter(Materials, UText3DProjectSettings::FMaterialParameters::GradientColorA, GradientColorA);
			SetVectorParameter(Materials, UText3DProjectSettings::FMaterialParameters::GradientColorB, GradientColorB);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::GradientOffset, GradientOffset);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::GradientSmoothness, GradientSmoothness);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::GradientRotation, GradientRotation);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::Mode, 1);
			break;
		case EText3DMaterialStyle::Texture:
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::TexturedUTiling, TextureTiling.X);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::TexturedVTiling, TextureTiling.Y);
			SetTextureParameter(Materials, UText3DProjectSettings::FMaterialParameters::MainTexture, TextureAsset);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::Mode, 2);
			break;
		}

		if (BlendMode == EText3DMaterialBlendMode::Translucent || Style == EText3DMaterialStyle::Gradient)
		{
			if (EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Geometry)
				|| EnumHasAnyFlags(InParameters.UpdateFlags, EText3DRendererFlags::Layout))
			{
				// Convert to local space
				FBox TextBounds = Text3DComponent->GetBounds();
				const FTransform ComponentTransform = Text3DComponent->GetComponentTransform();
				TextBounds = TextBounds.TransformBy(ComponentTransform.Inverse());
				// Apply scale ratio based on font size 64 => 1.f
				FTransform RatioTransform;
				RatioTransform.SetScale3D(FVector(64.f / FMath::Max(Text3DComponent->GetFontSize(), 0.1f)));
				LocalTextBounds = TextBounds.TransformBy(RatioTransform);
			}

			SetVectorParameter(Materials, UText3DProjectSettings::FMaterialParameters::BoundsOrigin, LocalTextBounds.Min);
			SetVectorParameter(Materials, UText3DProjectSettings::FMaterialParameters::BoundsSize, LocalTextBounds.GetSize());
		}

		if (BlendMode == EText3DMaterialBlendMode::Translucent)
		{
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::Opacity, Opacity);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::MaskEnabled, bUseMask ? 1 : 0);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::MaskOffset, MaskOffset);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::MaskRotation, MaskRotation);
			SetScalarParameter(Materials, UText3DProjectSettings::FMaterialParameters::MaskSmoothness, MaskSmoothness);
		}

		FrontMaterial = DynFrontMaterial;
		BackMaterial = DynBackMaterial;
		ExtrudeMaterial = DynExtrudeMaterial;
		BevelMaterial = DynBevelMaterial;
	}

	if (!MaterialOverrides.IsEmpty())
	{
		const UText3DStyleExtensionBase* StyleExtension = Text3DComponent->GetStyleExtension();

		if (!StyleExtension)
		{
			UE_LOGF(LogText3D, Warning, "Invalid style extension retrieved in material extension, cannot proceed.")
			return EText3DExtensionResult::Failed;
		}

		for (FText3DMaterialOverride& MaterialOverride : MaterialOverrides)
		{
			if (const UText3DStyleBase* StyleRule = StyleExtension->GetStyle(MaterialOverride.Tag))
			{
				UMaterialInterface* Material = FrontMaterial;

				if (StyleRule->GetOverrideFrontColor())
				{
					if (UMaterialInstanceDynamic* DynamicMaterial = FindOrAdd(EText3DGroupType::Front, MaterialOverride.Tag))
					{
						const FLinearColor SolidColor = StyleRule->GetFrontColor();
#if WITH_EDITOR
						// Workaround to trigger an update of the material (eg: after a save operation, for spawnable)
						DynamicMaterial->ClearParameterValues();
#endif
						DynamicMaterial->SetVectorParameterValue(UText3DProjectSettings::FMaterialParameters::SolidColor, SolidColor);
						DynamicMaterial->SetScalarParameterValue(UText3DProjectSettings::FMaterialParameters::Mode, 0);
						DynamicMaterial->SetScalarParameterValue(UText3DProjectSettings::FMaterialParameters::Opacity, Opacity);
						Material = DynamicMaterial;
					}
				}

				MaterialOverride.Materials[static_cast<int32>(EText3DGroupType::Front)] = Material;
				MaterialOverride.Materials[static_cast<int32>(EText3DGroupType::Bevel)] = BevelMaterial;
				MaterialOverride.Materials[static_cast<int32>(EText3DGroupType::Extrude)] = ExtrudeMaterial;
				MaterialOverride.Materials[static_cast<int32>(EText3DGroupType::Back)] = BackMaterial;
			}
		}
	}

	return EText3DExtensionResult::Finished;
}

EText3DExtensionResult UText3DDefaultMaterialExtension::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Active;
}

void UText3DDefaultMaterialExtension::SetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
{
	if (!InParameters.Tag.IsNone())
	{
		FText3DMaterialOverride* MaterialOverride = MaterialOverrides.FindByPredicate([&InParameters](const FText3DMaterialOverride& InMaterialOverride)
		{
			return InMaterialOverride.Tag.IsEqual(InParameters.Tag);
		});

		if (MaterialOverride)
		{
			const int32 Index = static_cast<int32>(InParameters.Group);

			if (MaterialOverride->Materials[Index] != InMaterial)
			{
				MaterialOverride->Materials[Index] = InMaterial;
				OnMaterialOptionsChanged();
			}

			return;
		}
	}

	switch (InParameters.Group)
	{
	case EText3DGroupType::Front:
		SetFrontMaterial(InMaterial);
		break;
	case EText3DGroupType::Bevel:
		SetBevelMaterial(InMaterial);
		break;
	case EText3DGroupType::Extrude:
		SetExtrudeMaterial(InMaterial);
		break;
	case EText3DGroupType::Back:
		SetBackMaterial(InMaterial);
		break;
	default:
		return;
	}
}

UMaterialInterface* UText3DDefaultMaterialExtension::GetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters) const
{
	if (!InParameters.Tag.IsNone())
	{
		const FText3DMaterialOverride* MaterialOverride = MaterialOverrides.FindByPredicate([&InParameters](const FText3DMaterialOverride& InMaterialOverride)
		{
			return InMaterialOverride.Tag.IsEqual(InParameters.Tag);
		});

		const int32 MaterialIndex = static_cast<int32>(InParameters.Group);
		if (MaterialOverride && MaterialOverride->Materials.IsValidIndex(MaterialIndex))
		{
			return MaterialOverride->Materials[MaterialIndex];
		}
	}

	switch (InParameters.Group)
	{
	case EText3DGroupType::Front:
		return FrontMaterial;
	case EText3DGroupType::Bevel:
		return BevelMaterial;
	case EText3DGroupType::Extrude:
		return ExtrudeMaterial;
	case EText3DGroupType::Back:
		return BackMaterial;
	default:
		return nullptr;
	}
}

int32 UText3DDefaultMaterialExtension::GetMaterialCount() const
{
	return static_cast<int32>(EText3DGroupType::TypeCount) * (MaterialOverrides.Num() + 1); 
}

void UText3DDefaultMaterialExtension::GetMaterialNames(TArray<FName>& OutNames) const
{
	OutNames.Reset(GetMaterialCount());

	// Default
	OutNames.Add(NAME_None);
	Algo::Transform(MaterialOverrides, OutNames, [](const FText3DMaterialOverride& InMaterialOverride)
	{
		return InMaterialOverride.Tag;
	});
}

UMaterialInstanceDynamic* UText3DDefaultMaterialExtension::FindOrAdd(EText3DGroupType InGroup, FName InTag)
{
	using namespace UE::Text3D::Material;

	const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get();
	check(!!Text3DSettings);

	const FText3DMaterialKey MaterialKey(BlendMode, bIsUnlit);
	UMaterial* ParentMaterial = Text3DSettings->GetBaseMaterial(MaterialKey);

	if (Style != EText3DMaterialStyle::Solid)
	{
		FMaterialParameters Parameters;
		Parameters.Group = InGroup;
		Parameters.Tag = InTag;
		if (UMaterialInstanceDynamic* DynMat = Cast<UMaterialInstanceDynamic>(GetMaterial(Parameters)))
		{
			if (DynMat->GetBaseMaterial() == ParentMaterial)
			{
				return DynMat;
			}
		}
	}

	if (Style == EText3DMaterialStyle::Custom || Style == EText3DMaterialStyle::Invalid)
	{
		return nullptr;
	}

	FText3DMaterialGroupKey GroupKey(MaterialKey, InGroup, InTag);
	if (const TObjectPtr<UMaterialInstanceDynamic>* Material = GroupDynamicMaterials.Find(GroupKey))
	{
		if (*Material)
		{
			return *Material;
		}
	}

	UMaterialInstanceDynamic* DynamicMaterial = UMaterialInstanceDynamic::Create(ParentMaterial, this);
	if (DynamicMaterial)
	{
		DynamicMaterial->SetFlags(RF_Transient | RF_Public);
		GroupDynamicMaterials.Emplace(GroupKey, DynamicMaterial);
	}

	return DynamicMaterial;
}

UMaterialInstanceDynamic* UText3DDefaultMaterialExtension::Find(EText3DGroupType InGroup, FName InTag) const
{
	const FText3DMaterialKey MaterialKey(BlendMode, bIsUnlit);
	const FText3DMaterialGroupKey GroupKey(MaterialKey, InGroup, InTag);
	const TObjectPtr<UMaterialInstanceDynamic>* Material = GroupDynamicMaterials.Find(GroupKey);
	return Material ? *Material : nullptr;
}

void UText3DDefaultMaterialExtension::SetVectorParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, FVector InValue)
{
	for (UMaterialInstanceDynamic* Material : InMaterials)
	{
		Material->SetVectorParameterValue(InKey, InValue);
	}
}

void UText3DDefaultMaterialExtension::SetVectorParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, FLinearColor InValue)
{
	for (UMaterialInstanceDynamic* Material : InMaterials)
	{
		Material->SetVectorParameterValue(InKey, InValue);
	}
}

void UText3DDefaultMaterialExtension::SetScalarParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, float InValue)
{
	for (UMaterialInstanceDynamic* Material : InMaterials)
	{
		Material->SetScalarParameterValue(InKey, InValue);
	}
}

void UText3DDefaultMaterialExtension::SetTextureParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, UTexture* InValue)
{
	for (UMaterialInstanceDynamic* Material : InMaterials)
	{
		Material->SetTextureParameterValue(InKey, InValue);
	}
}

void UText3DDefaultMaterialExtension::OnMaterialOptionsChanged()
{
	RequestUpdate(EText3DRendererFlags::Material);
}

void UText3DDefaultMaterialExtension::OnCustomMaterialChanged()
{
	if (bUseSingleMaterial && Style == EText3DMaterialStyle::Custom)
	{
		BackMaterial = FrontMaterial;
		BevelMaterial = FrontMaterial;
		ExtrudeMaterial = FrontMaterial;
	}

	OnMaterialOptionsChanged();
}

void UText3DDefaultMaterialExtension::ClearParameterValues()
{
	auto ClearParamValues = [](UMaterialInterface* InMaterial)
		{
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(InMaterial))
			{
				MID->ClearParameterValues();
			}
		};

	ClearParamValues(FrontMaterial);
	ClearParamValues(BackMaterial);
	ClearParamValues(BevelMaterial);
	ClearParamValues(ExtrudeMaterial);

	for (const FText3DMaterialOverride& Override : MaterialOverrides)
	{
		for (UMaterialInterface* Material : Override.Materials)
		{
			ClearParamValues(Material);
		}
	}

	for (const TPair<FText3DMaterialGroupKey, TObjectPtr<UMaterialInstanceDynamic>>& Pair : GroupDynamicMaterials)
	{
		ClearParamValues(Pair.Value);
	}
}

void UText3DDefaultMaterialExtension::RegisterMaterialOverride(FName InTag)
{
	if (!InTag.IsNone())
	{
		FText3DMaterialOverride MaterialOverride
		{
			.Tag = InTag
		};

		MaterialOverride.Materials.SetNum(static_cast<int32>(EText3DGroupType::TypeCount));

		MaterialOverrides.AddUnique(MaterialOverride);
	}
}

void UText3DDefaultMaterialExtension::UnregisterMaterialOverride(FName InTag)
{
	if (!InTag.IsNone())
	{
		MaterialOverrides.RemoveAll([InTag](const FText3DMaterialOverride& InMaterialOverride)
		{
			return InMaterialOverride.Tag.IsEqual(InTag);
		});
	}
}

void UText3DDefaultMaterialExtension::ForEachMaterial(TFunctionRef<bool(const UE::Text3D::Material::FMaterialParameters&, UMaterialInterface*)> InFunctor) const
{
	// Base materials
	using namespace UE::Text3D::Material;

	FMaterialParameters Params;
	for (int32 Index = 0; Index < static_cast<int32>(EText3DGroupType::TypeCount); Index++)
	{
		const EText3DGroupType GroupType = static_cast<EText3DGroupType>(Index);
		Params.Group = GroupType;
		if (!InFunctor(Params, GetMaterial(Params)))
		{
			return;
		}
	}

	// Override materials
	for (const FText3DMaterialOverride& MaterialOverride : MaterialOverrides)
	{
		Params.Tag = MaterialOverride.Tag;
		for (TConstEnumerateRef<TObjectPtr<UMaterialInterface>> Material : EnumerateRange(MaterialOverride.Materials))
		{
			Params.Group = static_cast<EText3DGroupType>(Material.GetIndex());
			if (!InFunctor(Params, *Material))
			{
				return;
			}
		}
	}
}

FVector UText3DDefaultMaterialExtension::GetGradientDirection() const
{
	const UText3DComponent* Text3DComponent = GetText3DComponent();
	FVector GradientDir = Text3DComponent->GetUpVector().RotateAngleAxis(-GradientRotation * 360, Text3DComponent->GetForwardVector());

	/**
	 * In order to properly map gradient offset along the text surface, text bounds are not normalized (anymore) in the material function creating the gradient.
	 * Therefore, we need to remap the gradient direction, taking into account the current text actor bounds.
	 */
	FVector Origin;
	FVector Extent;
	Text3DComponent->GetBounds(Origin, Extent);

	const FVector GradientDirFixer = FVector(1.0f, Extent.Z, Extent.Y);
	GradientDir *= GradientDirFixer;

	GradientDir.Normalize();

	return GradientDir;
}

void UText3DDefaultMaterialExtension::PreCacheMaterials()
{
	if (UMaterialInstanceDynamic* NewFrontMaterial = FindOrAdd(EText3DGroupType::Front))
	{
		FrontMaterial = NewFrontMaterial;
	}

	if (UMaterialInstanceDynamic* NewBackMaterial = FindOrAdd(EText3DGroupType::Back))
	{
		BackMaterial = NewBackMaterial;
	}
	
	if (UMaterialInstanceDynamic* NewExtrudeMaterial = FindOrAdd(EText3DGroupType::Extrude))
	{
		ExtrudeMaterial = NewExtrudeMaterial;
	}
	
	if (UMaterialInstanceDynamic* NewBevelMaterial = FindOrAdd(EText3DGroupType::Bevel))
	{
		BevelMaterial = NewBevelMaterial;
	}
}

void UText3DDefaultMaterialExtension::PostLoad()
{
	Super::PostLoad();
	
	PreCacheMaterials();
}

void UText3DDefaultMaterialExtension::PostInitProperties()
{
	Super::PostInitProperties();

	GroupDynamicMaterials.Empty();
	if (!IsInAsyncLoadingThread())
	{
		// This needs to be called on the game thread. In the case of loading it will get called in PostLoad()
		PreCacheMaterials();
	}
}

void UText3DDefaultMaterialExtension::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (bInDuplicateForPIE)
	{
		// Clear MID parameters as these get duplicated but not updated correctly for PIE.
		ClearParameterValues();
	}
}

#if WITH_EDITOR
void UText3DDefaultMaterialExtension::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, FrontColor),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, BackColor),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, ExtrudeColor),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, BevelColor),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, GradientColorA),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, GradientColorB),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, GradientSmoothness),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, GradientOffset),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, GradientRotation),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, TextureAsset),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, TextureTiling),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, BlendMode),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, bIsUnlit),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, Opacity),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, bUseMask),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, MaskOffset),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, MaskSmoothness),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, MaskRotation),
	};

	static const TSet<FName> CustomPropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, Style),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, bUseSingleMaterial),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, FrontMaterial),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, BevelMaterial),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, ExtrudeMaterial),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultMaterialExtension, BackMaterial),
	};

	const FName MemberPropertyName = InEvent.GetMemberPropertyName();

	if (CustomPropertyNames.Contains(MemberPropertyName))
	{
		OnCustomMaterialChanged();
	}
	else if (PropertyNames.Contains(MemberPropertyName))
	{
		OnMaterialOptionsChanged();
	}
}

void UText3DDefaultMaterialExtension::PostEditUndo()
{
	Super::PostEditUndo();
	OnMaterialOptionsChanged();
}
#endif
