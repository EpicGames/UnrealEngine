// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultRenderingExtension.h"

void UText3DDefaultRenderingExtension::SetCastShadow(bool bInCastShadow)
{
	if (bCastShadow == bInCastShadow)
	{
		return;
	}
	
	bCastShadow = bInCastShadow;
	OnRenderingOptionsChanged();
}

void UText3DDefaultRenderingExtension::SetCastHiddenShadow(bool bInCastShadow)
{
	if (bCastHiddenShadow == bInCastShadow)
	{
		return;
	}
	
	bCastHiddenShadow = bInCastShadow;
	OnRenderingOptionsChanged();
}

void UText3DDefaultRenderingExtension::SetAffectDynamicIndirectLighting(bool bInValue)
{
	if (bAffectDynamicIndirectLighting == bInValue)
	{
		return;
	}

	bAffectDynamicIndirectLighting = bInValue;
	OnRenderingOptionsChanged();
}

void UText3DDefaultRenderingExtension::SetAffectIndirectLightingWhileHidden(bool bInValue)
{
	if (bAffectIndirectLightingWhileHidden == bInValue)
	{
		return;
	}

	bAffectIndirectLightingWhileHidden = bInValue;
	OnRenderingOptionsChanged();
}

void UText3DDefaultRenderingExtension::SetHoldout(bool bInHoldout)
{
	if (bHoldout == bInHoldout)
	{
		return;
	}

	bHoldout = bInHoldout;
	OnRenderingOptionsChanged();
}

#if WITH_EDITOR
void UText3DDefaultRenderingExtension::PostEditUndo()
{
	Super::PostEditUndo();
	OnRenderingOptionsChanged();
}

void UText3DDefaultRenderingExtension::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DDefaultRenderingExtension, bCastShadow),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultRenderingExtension, bCastHiddenShadow),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultRenderingExtension, bAffectDynamicIndirectLighting),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultRenderingExtension, bAffectIndirectLightingWhileHidden),
		GET_MEMBER_NAME_CHECKED(UText3DDefaultRenderingExtension, bHoldout)
	};

	if (PropertyNames.Contains(InEvent.GetMemberPropertyName()))
	{
		OnRenderingOptionsChanged();
	}
}
#endif

bool UText3DDefaultRenderingExtension::GetTextCastShadow() const
{
	return GetCastShadow();
}

bool UText3DDefaultRenderingExtension::GetTextCastHiddenShadow() const
{
	return GetCastHiddenShadow();
}

bool UText3DDefaultRenderingExtension::GetTextAffectDynamicIndirectLighting() const
{
	return GetAffectDynamicIndirectLighting();
}

bool UText3DDefaultRenderingExtension::GetTextAffectIndirectLightingWhileHidden() const
{
	return GetAffectIndirectLightingWhileHidden();
}

bool UText3DDefaultRenderingExtension::GetTextHoldout() const
{
	return GetHoldout();
}

void UText3DDefaultRenderingExtension::OnRenderingOptionsChanged()
{
	RequestUpdate(EText3DRendererFlags::Visibility);
}
