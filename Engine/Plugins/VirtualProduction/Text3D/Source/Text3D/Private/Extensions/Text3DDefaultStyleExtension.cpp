// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultStyleExtension.h"

#include "Engine/Font.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "Fonts/SlateFontInfo.h"
#include "Settings/Text3DProjectSettings.h"
#include "Styles/Text3DStyleBase.h"
#include "Styles/Text3DStyleSet.h"
#include "Styling/SlateStyle.h"
#include "Text3DComponent.h"

UText3DDefaultStyleExtension::UText3DDefaultStyleExtension()
{
	if (const UText3DProjectSettings* TextSettings = UText3DProjectSettings::Get())
	{
		StyleSet = TextSettings->GetDefaultStyleSet();		
	}

	UText3DStyleSet::OnStyleSetUpdated().AddUObject(this, &UText3DDefaultStyleExtension::OnStyleSetUpdated);
}

void UText3DDefaultStyleExtension::SetStyleSet(UText3DStyleSet* InStyleSet)
{
	if (StyleSet == InStyleSet)
	{
		return;
	}

	StyleSet = InStyleSet;
	OnStyleSetChanged();
}

#if WITH_EDITOR
void UText3DDefaultStyleExtension::PostEditUndo()
{
	Super::PostEditUndo();
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DDefaultStyleExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DDefaultStyleExtension, Styles))
	{
		if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear
			|| InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
		{
			RequestUpdate(EText3DRendererFlags::All);
		}
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DDefaultStyleExtension, StyleSet))
	{
		OnStyleSetChanged();
	}
}
#endif

TSharedPtr<FTextBlockStyle> UText3DDefaultStyleExtension::GetDefaultStyle() const
{
	return DefaultFontStyle;
}

TSharedPtr<FSlateStyleSet> UText3DDefaultStyleExtension::GetCustomStyles() const
{
	return TextStyleSet;
}

UText3DStyleBase* UText3DDefaultStyleExtension::GetStyle(FName InStyleName) const
{
	auto FindStyleFunctor = [InStyleName](UText3DStyleBase* InStyle)->bool
		{
			return InStyle && InStyle->GetStyleName() == InStyleName;
		};

	// Look for overrides first
	if (const TObjectPtr<UText3DStyleBase>* FormatStyle = Styles.FindByPredicate(FindStyleFunctor))
	{
		return *FormatStyle;
	}

	if (StyleSet)
	{
		if (const TObjectPtr<UText3DStyleBase>* FormatStyle = StyleSet->GetStyles().FindByPredicate(FindStyleFunctor))
		{
			return *FormatStyle;
		}
	}

	return nullptr;
}

void UText3DDefaultStyleExtension::PrepareBuild(EText3DRendererFlags InUpdateFlags)
{
	Super::PrepareBuild(InUpdateFlags);

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultStyleExtension::PrepareRebuild);

	if (!DefaultFontStyle.IsValid()
		|| !TextStyleSet.IsValid()
		|| EnumHasAnyFlags(InUpdateFlags, EText3DRendererFlags::Geometry))
	{
		const UText3DComponent* Text3DComponent = GetText3DComponent();
		const UFont* Font = Text3DComponent->GetFont();
		UText3DMaterialExtensionBase* MaterialExtension = Text3DComponent->GetMaterialExtension();

		if (!DefaultFontStyle.IsValid())
		{
			DefaultFontStyle = MakeShared<FTextBlockStyle>();
		}

		FSlateFontInfo FontInfo(Font, Text3DComponent->GetFontSize());
		FontInfo.CompositeFont = Font && Font->GetCompositeFont() ? MakeShared<FCompositeFont>(*Font->GetCompositeFont()) : FStyleDefaults::GetFontInfo().CompositeFont;
		FontInfo.TypefaceFontName = Text3DComponent->GetTypeface();
		FontInfo.FontFallback = EFontFallback::FF_NoFallback;
		DefaultFontStyle->SetFont(FontInfo);

		TextStyleSet = MakeShared<FSlateStyleSet>(TEXT("Text3DRichTextStyle"));
		TextStyleSet->Set(FName(TEXT("default")), *DefaultFontStyle);

		TArray<FName> OutdatedMaterialNames;
		MaterialExtension->GetMaterialNames(OutdatedMaterialNames);

		// Process style set first
		if (StyleSet)
		{
			for (const UText3DStyleBase* FormatStyle : StyleSet->GetStyles())
			{
				if (FormatStyle)
				{
					const FName StyleName = FormatStyle->GetStyleName();
					if (!StyleName.IsNone())
					{
						FTextBlockStyle TextStyle = *DefaultFontStyle;
						FormatStyle->ConfigureStyle(TextStyle);
						TextStyleSet->Set(StyleName, TextStyle);
						MaterialExtension->RegisterMaterialOverride(StyleName);
						OutdatedMaterialNames.Remove(StyleName);
					}
				}
			}
		}

		// Process additional styles to override style set
		for (const UText3DStyleBase* FormatStyle : Styles)
		{
			if (FormatStyle)
			{
				const FName StyleName = FormatStyle->GetStyleName();
				if (!StyleName.IsNone())
				{
					FTextBlockStyle TextStyle = *DefaultFontStyle;
					FormatStyle->ConfigureStyle(TextStyle);
					TextStyleSet->Set(StyleName, TextStyle);
					MaterialExtension->RegisterMaterialOverride(StyleName);
					OutdatedMaterialNames.Remove(StyleName);
				}
			}
		}

		// Unregister outdated material slot
		for (const FName MaterialName : OutdatedMaterialNames)
		{
			MaterialExtension->UnregisterMaterialOverride(MaterialName);
		}
	}
}

EText3DExtensionResult UText3DDefaultStyleExtension::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{

	return EText3DExtensionResult::Finished;
}

EText3DExtensionResult UText3DDefaultStyleExtension::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Active;
}

void UText3DDefaultStyleExtension::OnStyleSetUpdated(UText3DStyleSet* InStyleSet)
{
	if (StyleSet == InStyleSet)
	{
		OnStyleSetChanged();
	}
}

void UText3DDefaultStyleExtension::OnStyleSetChanged()
{
	RequestUpdate(EText3DRendererFlags::All);
}
