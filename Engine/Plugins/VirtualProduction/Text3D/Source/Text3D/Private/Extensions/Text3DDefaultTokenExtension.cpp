// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DDefaultTokenExtension.h"

#include "Internationalization/TextFormatter.h"
#include "Text3DComponent.h"
#include "Tokens/Text3DTokenBase.h"

#if WITH_EDITOR
void UText3DDefaultTokenExtension::PostEditUndo()
{
	Super::PostEditUndo();
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DDefaultTokenExtension::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DDefaultTokenExtension, Tokens))
	{
		if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear
			|| InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove
			|| InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayMove)
		{
			RequestUpdate(EText3DRendererFlags::All);
		}
	}
}
#endif

const FText& UText3DDefaultTokenExtension::GetFormattedText() const
{
	return FormattedText;
}

void UText3DDefaultTokenExtension::PrepareBuild(EText3DRendererFlags InUpdateFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DDefaultTokenExtension::PrepareRebuild);

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	FormattedText = Text3DComponent->GetText();

	FFormatNamedArguments NamedArguments;
	NamedArguments.Reserve(Tokens.Num() * 2);
	for (UText3DTokenBase* Token : Tokens)
	{
		if (Token)
		{
			Token->CollectTokens(NamedArguments);
		}
	}

	constexpr bool bRebuildText = false;
	constexpr bool bRebuildAsSource = false;
	FormattedText = FTextFormatter::Format(MoveTemp(FormattedText), MoveTemp(NamedArguments), bRebuildText, bRebuildAsSource);

	// For backward compatibility
	Text3DComponent->FormatText(FormattedText);
}

EText3DExtensionResult UText3DDefaultTokenExtension::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Finished;
}

EText3DExtensionResult UText3DDefaultTokenExtension::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Active;
}
