// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DEffectExtensionBase.h"
#include "Text3DComponent.h"

EText3DExtensionResult UText3DEffectExtensionBase::PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	if (InParameters.CurrentFlag != EText3DRendererFlags::Layout)
	{
		return EText3DExtensionResult::Active;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();

	const uint16 CharacterCount = Text3DComponent->GetCharacterCount();
	for (uint16 Index = 0; Index < CharacterCount; Index++)
	{
		if (GetTargetRange().IsInRange(Index))
		{
			ApplyEffect(Index, CharacterCount);
		}
	}

	return EText3DExtensionResult::Finished;
}

EText3DExtensionResult UText3DEffectExtensionBase::PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	return EText3DExtensionResult::Active;
}