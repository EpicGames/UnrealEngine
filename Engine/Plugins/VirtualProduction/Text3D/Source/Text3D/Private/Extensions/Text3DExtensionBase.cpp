// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DExtensionBase.h"

#include "Text3DComponent.h"

void UText3DExtensionBase::RequestUpdate(EText3DRendererFlags InFlags, bool bInImmediateUpdate) const
{
	if (UText3DComponent* Text3DComponent = GetText3DComponent())
	{
		Text3DComponent->RequestUpdate(InFlags, bInImmediateUpdate);
	}
}

UText3DComponent* UText3DExtensionBase::GetText3DComponent() const
{
	return GetTypedOuter<UText3DComponent>();
}
