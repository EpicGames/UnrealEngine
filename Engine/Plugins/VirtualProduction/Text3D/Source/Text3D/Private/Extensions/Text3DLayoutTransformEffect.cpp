// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/Text3DLayoutTransformEffect.h"

#include "Characters/Text3DCharacterBase.h"
#include "Curves/CurveFloat.h"
#include "Text3DComponent.h"

void UText3DLayoutTransformEffect::SetLocationEnabled(bool bEnabled)
{
	if (bLocationEnabled != bEnabled)
	{
		bLocationEnabled = bEnabled;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetLocationProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(LocationProgress, Progress))
	{
		LocationProgress = Progress;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetLocationOrder(EText3DCharacterEffectOrder Order)
{
	if (LocationOrder != Order)
	{
		LocationOrder = Order;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetLocationBegin(const FVector& InBegin)
{
	if (!LocationBegin.Equals(InBegin))
	{
		LocationBegin = InBegin;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetLocationEnd(const FVector& InEnd)
{
	if (!LocationEnd.Equals(InEnd))
	{
		LocationEnd = InEnd;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetLocationEaseCurve(UCurveFloat* InEaseCurve)
{
	if (LocationEaseCurve == InEaseCurve)
	{
		return;
	}

	LocationEaseCurve = InEaseCurve;
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::SetLocationStagger(bool bInStagger)
{
	if (bLocationStagger == bInStagger)
	{
		return;
	}

	bLocationStagger = bInStagger;
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::SetScaleEnabled(bool bEnabled)
{
	if (bScaleEnabled != bEnabled)
	{
		bScaleEnabled = bEnabled;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetScaleProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(ScaleProgress, Progress))
	{
		ScaleProgress = Progress;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetScaleOrder(EText3DCharacterEffectOrder Order)
{
	if (ScaleOrder != Order)
	{
		ScaleOrder = Order;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetScaleBegin(const FVector& Value)
{
	if (!ScaleBegin.Equals(Value))
	{
		ScaleBegin = Value;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetScaleEnd(const FVector& Value)
{
	if (!ScaleEnd.Equals(Value))
	{
		ScaleEnd = Value;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetScaleEaseCurve(UCurveFloat* InEaseCurve)
{
	if (ScaleEaseCurve == InEaseCurve)
	{
		return;
	}

	ScaleEaseCurve = InEaseCurve;
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::SetScaleStagger(bool bInStagger)
{
	if (bScaleStagger == bInStagger)
	{
		return;
	}

	bScaleStagger = bInStagger;
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::SetRotationEnabled(bool bEnabled)
{
	if (bRotationEnabled != bEnabled)
	{
		bRotationEnabled = bEnabled;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetRotationProgress(float Progress)
{
	Progress = FMath::Clamp(Progress, 0.0f, 100.0f);
	if (!FMath::IsNearlyEqual(RotationProgress, Progress))
	{
		RotationProgress = Progress;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetRotationOrder(EText3DCharacterEffectOrder Order)
{
	if (RotationOrder != Order)
	{
		RotationOrder = Order;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetRotationBegin(const FRotator& Value)
{
	if (!RotationBegin.Equals(Value))
	{
		RotationBegin = Value;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetRotationEnd(const FRotator& Value)
{
	if (!RotationEnd.Equals(Value))
	{
		RotationEnd = Value;
		OnTransformOptionsChanged();
	}
}

void UText3DLayoutTransformEffect::SetRotationEaseCurve(UCurveFloat* InEaseCurve)
{
	if (RotationEaseCurve == InEaseCurve)
	{
		return;
	}

	RotationEaseCurve = InEaseCurve;
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::SetRotationStagger(bool bInStagger)
{
	if (bRotationStagger == bInStagger)
	{
		return;
	}

	bRotationStagger = bInStagger;
	OnTransformOptionsChanged();
}

UText3DLayoutTransformEffect::UText3DLayoutTransformEffect()
{
	OnTransformOptionsChanged();
}

#if WITH_EDITOR
void UText3DLayoutTransformEffect::PostEditUndo()
{
	Super::PostEditUndo();
	OnTransformOptionsChanged();
}

void UText3DLayoutTransformEffect::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, bLocationEnabled),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, LocationProgress),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, LocationOrder),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, LocationBegin),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, LocationEnd),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, LocationEaseCurve),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, bRotationEnabled),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationProgress),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationOrder),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationBegin),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationEnd),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationEnd),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, RotationEaseCurve),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, bScaleEnabled),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, ScaleProgress),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, ScaleOrder),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, ScaleBegin),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, ScaleEnd),
		GET_MEMBER_NAME_CHECKED(UText3DLayoutTransformEffect, ScaleEaseCurve),
	};

	if (PropertyNames.Contains(InEvent.GetMemberPropertyName()))
	{
		OnTransformOptionsChanged();
	}
}
#endif

void UText3DLayoutTransformEffect::OnTransformOptionsChanged()
{
	EText3DRendererFlags Flags = EText3DRendererFlags::Material;
	EnumAddFlags(Flags, EText3DRendererFlags::Layout);
	RequestUpdate(Flags);
}

void UText3DLayoutTransformEffect::ApplyEffect(uint32 InGlyphIndex, uint32 InGlyphCount)
{
	if (!bLocationEnabled && !bRotationEnabled && !bScaleEnabled)
	{
		return;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	UText3DCharacterBase* Character = Text3DComponent->GetCharacter(InGlyphIndex);

	constexpr bool bReset = false;
	FTransform& CharacterTransform = Character->GetTransform(bReset);

	InGlyphIndex += 1;
	InGlyphCount += 1;

	if (bLocationEnabled)
	{
		const float Effect = CalculateEffect(InGlyphIndex, InGlyphCount, LocationOrder, LocationProgress, LocationEaseCurve, bLocationStagger);
		const FVector Location = LocationBegin + (LocationEnd - LocationBegin) * Effect;
		CharacterTransform.AddToTranslation(Location);
	}

	if (bScaleEnabled)
	{
		const float Effect = CalculateEffect(InGlyphIndex, InGlyphCount, ScaleOrder, ScaleProgress, ScaleEaseCurve, bScaleStagger);
		const FVector Scale = ScaleBegin + (ScaleEnd - ScaleBegin) * Effect;
		CharacterTransform.MultiplyScale3D(Scale.ComponentMax(FVector(UE_KINDA_SMALL_NUMBER)));
	}

	if (bRotationEnabled)
	{
		const float Effect = CalculateEffect(InGlyphIndex, InGlyphCount, RotationOrder, RotationProgress, RotationEaseCurve, bRotationStagger);
		const FRotator Rotator = RotationBegin + (RotationEnd - RotationBegin) * Effect;
		CharacterTransform.ConcatenateRotation(Rotator.Quaternion());
	}
}

int32 UText3DLayoutTransformEffect::GetEffectPosition(int32 Index, int32 Total, EText3DCharacterEffectOrder Order) const
{
	int32 Center = 0.5f * Total - 0.5f;
	switch (Order)
	{
	case EText3DCharacterEffectOrder::FromCenter:
		{
			if (Index > Center)
			{
				Index = Total - Index - 1;
			}

			return Center - Index;
		}

	case EText3DCharacterEffectOrder::ToCenter:
		{
			if (Index > Center)
			{
				return Total - Index - 1;
			}

			return Index;
		}

	case EText3DCharacterEffectOrder::Opposite:
		{
			return Total - Index - 1;
		}
	default: ;
	}

	return Index;
}

float UText3DLayoutTransformEffect::CalculateEffect(int32 InIndex, int32 InTotal, EText3DCharacterEffectOrder InOrder, float InProgress, const UCurveFloat* InEaseCurve, bool bInStagger) const
{
	const int32 Position = GetEffectPosition(InIndex, InTotal, InOrder);
	const float NormalizedProgress = FMath::Clamp(InProgress * 0.01f, 0.0f, 1.0f);
	const float StaggerAmount = 1.0f / FMath::Max(1, InTotal);
	const float CharacterStart = Position * StaggerAmount;
	const float StaggerProgress = bInStagger ? StaggerAmount : 1.0f - CharacterStart;
	const float CharacterProgress = FMath::Clamp((NormalizedProgress - CharacterStart) / StaggerProgress, 0.0f, 1.0f);
	float CharacterEased = CharacterProgress;

	if (InEaseCurve)
	{
		float StartTime = 0.0f;
		float EndTime = 0.0f;
		InEaseCurve->FloatCurve.GetTimeRange(StartTime, EndTime);

		const float NormalizedTime = FMath::GetMappedRangeValueClamped(FVector2D(0.f, 1.f), FVector2D(StartTime, EndTime), CharacterProgress);
		CharacterEased = InEaseCurve->FloatCurve.Eval(NormalizedTime);
	}

	return CharacterEased;
}
