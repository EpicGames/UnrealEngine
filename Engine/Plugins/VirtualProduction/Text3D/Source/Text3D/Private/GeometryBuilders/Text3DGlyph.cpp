// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyph.h"

#include "Engine/StaticMesh.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MeshDescription.h"
#include "PhysicsEngine/BodySetup.h"
#include "Text3DInternalTypes.h"
#include "UDynamicMesh.h"
#include "Subsystems/Text3DEngineSubsystem.h"

FText3DGlyph::FText3DGlyph()
	: StaticMeshAttributes(MeshDescription)
{
	StaticMeshAttributes.Register();
	Groups.SetNum(static_cast<int32>(EText3DGroupType::TypeCount));
	MeshDescription.ReserveNewPolygonGroups(Groups.Num());

	for (int32 Index = 0; Index < Groups.Num(); Index++)
	{
		MeshDescription.CreatePolygonGroup();
	}
}

void FText3DGlyph::Build(FText3DCachedMesh& InMesh, EText3DMeshType InMeshType, UMaterialInterface* InDefaultMaterial)
{
	// Both dynamic and static build static meshes
	checkf(InMesh.StaticMesh, TEXT("Invalid static mesh object provided to the build function"));
	checkf(InMeshType != EText3DMeshType::Dynamic || InMesh.DynamicMesh, TEXT("Invalid dynamic mesh object provided to the build function"));

	auto AddMaterial = [](UStaticMesh* InMesh, UMaterialInterface* InMaterial, int32 InIndex)
	{
		using namespace UE::Text3D::Material;

		const TConstArrayView<FName> SlotNames = GetSlotNames();
		check(SlotNames.IsValidIndex(InIndex))

		FName MaterialName = SlotNames[InIndex];

#if WITH_EDITORONLY_DATA
		FStaticMaterial& StaticMaterial = InMesh->GetStaticMaterials().Emplace_GetRef(InMaterial, MaterialName, MaterialName);
#else
		FStaticMaterial& StaticMaterial = InMesh->GetStaticMaterials().Emplace_GetRef(InMaterial, MaterialName);
#endif

		StaticMaterial.UVChannelData = FMeshUVChannelInfo(1.0f);

		return MaterialName;
	};

	for (int32 Index = 0; Index < Groups.Num(); Index++)
	{
		const FPolygonGroupID PolyGroup(Index);
		if (MeshDescription.GetNumPolygonGroupTriangles(PolyGroup) != 0)
		{
			StaticMeshAttributes.GetPolygonGroupMaterialSlotNames()[PolyGroup] = AddMaterial(InMesh.StaticMesh, InDefaultMaterial, Index);
		}
	}

	{
		TArray<const FMeshDescription*> MeshDescriptions;
		MeshDescriptions.Add(&MeshDescription);

		constexpr bool bAllowCpuAccess = true;

		UStaticMesh::FBuildMeshDescriptionsParams BuildParams;
		BuildParams.bCommitMeshDescription = true;
		BuildParams.bFastBuild = true;
		BuildParams.bAllowCpuAccess = bAllowCpuAccess;

		InMesh.StaticMesh->bAllowCPUAccess = bAllowCpuAccess;
		InMesh.StaticMesh->BuildFromMeshDescriptions(MeshDescriptions, BuildParams);

		// Setup collision
		if (UBodySetup* Body = InMesh.StaticMesh->GetBodySetup())
		{
			Body->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseComplexAsSimple;
		}
	}

	// Dynamic mesh building
	if (InMeshType == EText3DMeshType::Dynamic)
	{
		static constexpr FGeometryScriptMeshReadLOD StaticMeshLOD
		{
			.LODType = EGeometryScriptLODType::SourceModel
		};

		static constexpr FGeometryScriptCopyMeshFromAssetOptions ConversionParams
		{
			.bApplyBuildSettings = false,
			.bRequestTangents = false,
			.bIgnoreRemoveDegenerates = false,
		};

		EGeometryScriptOutcomePins OutResult;
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMesh(InMesh.StaticMesh, InMesh.DynamicMesh, ConversionParams, StaticMeshLOD, OutResult);
	}
}

const FMeshDescription& FText3DGlyph::GetMeshDescription() const
{
	return MeshDescription;
}

FMeshDescription& FText3DGlyph::GetMeshDescription()
{
	return MeshDescription;
}

FStaticMeshAttributes& FText3DGlyph::GetStaticMeshAttributes()
{
	return StaticMeshAttributes;
}

TText3DGroupList& FText3DGlyph::GetGroups()
{
	return Groups;
}
