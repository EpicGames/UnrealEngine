// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "Text3DTypes.h"

class UMaterial;
class UMaterialInterface;
class UStaticMesh;
struct FText3DCachedMesh;

struct FText3DPolygonGroup
{
	int32 FirstVertex;
	int32 FirstTriangle;
};

using TText3DGroupList = TArray<FText3DPolygonGroup, TFixedAllocator<static_cast<int32>(EText3DGroupType::TypeCount)>>;

class FText3DGlyph
{
public:
	FText3DGlyph();

	void Build(FText3DCachedMesh& InMesh, EText3DMeshType InMeshType, UMaterialInterface* InDefaultMaterial);

	const FMeshDescription& GetMeshDescription() const;
	FMeshDescription& GetMeshDescription();
	FStaticMeshAttributes& GetStaticMeshAttributes();
	TText3DGroupList& GetGroups();

private:
	FMeshDescription MeshDescription;
	FStaticMeshAttributes StaticMeshAttributes;
	TText3DGroupList Groups;
};
