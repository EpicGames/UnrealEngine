// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphArrangement.h"

using namespace UE::Geometry;

FText3DGlyphArrangement::FText3DGlyphArrangement(const FAxisAlignedBox2f& InBoundsHint)
    : PointHash(static_cast<double>(InBoundsHint.MaxDim()) / 64, -1)
{
}

void FText3DGlyphArrangement::Insert(const FSegment2f& Segment)
{
    const FVector2f A = Segment.StartPoint();
    const FVector2f B = Segment.EndPoint();

    InsertSegment({static_cast<double>(A.X), static_cast<double>(A.Y)}, {static_cast<double>(B.X), static_cast<double>(B.Y)}, VertexSnapTol);
}

int32 FText3DGlyphArrangement::InsertPoint(const FVector2d &InPoint, double InTol)
{
    int32 PIdx = FindExistingVertex(InPoint);
    if (PIdx > -1)
    {
        return -1;
    }

    // TODO: currently this tries to add the vertex on the closest edge below tolerance; we should instead insert at *every* edge below tolerance!  ... but that is more inconvenient to write
    FVector2d x = FVector2d::Zero(), y = FVector2d::Zero();
    double ClosestDistSq = InTol*InTol;
    int32 FoundEdgeToSplit = -1;
    for (int32 EID = 0, ExistingEdgeMax = Graph.MaxEdgeID(); EID < ExistingEdgeMax; EID++)
    {
        if (!Graph.IsEdge(EID))
        {
            continue;
        }

        Graph.GetEdgeV(EID, x, y);
        FSegment2d Seg(x, y);
        double DistSq = Seg.DistanceSquared(InPoint);
        if (DistSq < ClosestDistSq)
        {
            ClosestDistSq = DistSq;
            FoundEdgeToSplit = EID;
        }
    }
    if (FoundEdgeToSplit > -1)
    {
        FDynamicGraph2d::FEdgeSplitInfo splitInfo;
        EMeshResult result = Graph.SplitEdge(FoundEdgeToSplit, splitInfo);
        ensureMsgf(result == EMeshResult::Ok, TEXT("insert_into_segment: edge split failed?"));
        Graph.SetVertex(splitInfo.VNew, InPoint);
        PointHash.InsertPoint(splitInfo.VNew, InPoint);
        return splitInfo.VNew;
    }

    int32 VID = Graph.AppendVertex(InPoint);
    PointHash.InsertPoint(VID, InPoint);
    return VID;
}

bool FText3DGlyphArrangement::InsertSegment(FVector2d InA, FVector2d InB, double InTol)
{
    // handle degenerate edges
    int32 a_idx = FindExistingVertex(InA);
    int32 b_idx = FindExistingVertex(InB);
    if (a_idx == b_idx && a_idx >= 0)
    {
        return false;
    }
    // snap input vertices
    if (a_idx >= 0)
    {
        InA = Graph.GetVertex(a_idx);
    }
    if (b_idx >= 0)
    {
        InB = Graph.GetVertex(b_idx);
    }

    // handle tiny-segment case
    double SegLenSq = DistanceSquared(InA, InB);
    if (SegLenSq <= VertexSnapTol*VertexSnapTol)
    {
        // seg is too short and was already on an existing vertex; just consider that vertex to be the inserted segment
        if (a_idx >= 0 || b_idx >= 0)
        {
            return false;
        }
        // seg is too short and wasn't on an existing vertex; add it as an isolated vertex
        return InsertPoint(InA, InTol) != -1;
    }

    // ok find all intersections
    TArray<FIntersection> Hits;
    FindIntersectingEdges(InA, InB, Hits, InTol);

    // we are going to construct a list of <T,vertex_id> values along segment AB
    TArray<FSegmentPoint> points;
    FSegment2d segAB = FSegment2d(InA, InB);

    FindIntersectingVertices(segAB, a_idx, b_idx, points, InTol);

    // insert intersections into existing segments
    for (int32 i = 0, N = Hits.Num(); i < N; ++i)
    {
        FIntersection Intr = Hits[i];
        int32 EID = Intr.EID;
        double t0 = Intr.Intr.Parameter0, t1 = Intr.Intr.Parameter1;

        // insert first point at t0
        int32 new_eid = -1;
        if (Intr.Intr.Type == EIntersectionType::Point || Intr.Intr.Type == EIntersectionType::Segment)
        {
            FIndex2i new_info = SplitSegmentAtDistance(EID, t0, VertexSnapTol);
            new_eid = new_info.B;
            FVector2d v = Graph.GetVertex(new_info.A);
            points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
        }

        // if intersection was on-segment, then we have a second point at t1
        if (Intr.Intr.Type == EIntersectionType::Segment)
        {
            if (new_eid == -1)
            {
                // did not actually split edge for t0, so we can still use EID
                FIndex2i new_info = SplitSegmentAtDistance(EID, t1, VertexSnapTol);
                FVector2d v = Graph.GetVertex(new_info.A);
                points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
            }
            else
            {
                // find t1 was in EID, rebuild in new_eid
                FSegment2d new_seg = Graph.GetEdgeSegment(new_eid);
                FVector2d p1 = Intr.Intr.GetSegment1().PointAt(t1);
                double new_t1 = new_seg.Project(p1);
                // note: new_t1 may be outside of new_seg due to snapping; in this case the segment will just not be split

                FIndex2i new_info = SplitSegmentAtDistance(new_eid, new_t1, VertexSnapTol);
                FVector2d v = Graph.GetVertex(new_info.A);
                points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
            }
        }
    }

    // find or create start and end points
    if (a_idx == -1)
    {
        a_idx = FindExistingVertex(InA);
    }
    if (a_idx == -1)
    {
        a_idx = Graph.AppendVertex(InA);
        PointHash.InsertPoint(a_idx, InA);
    }
    if (b_idx == -1)
    {
        b_idx = FindExistingVertex(InB);
    }
    if (b_idx == -1)
    {
        b_idx = Graph.AppendVertex(InB);
        PointHash.InsertPoint(b_idx, InB);
    }

    // add start/end to points list. These may be duplicates but we will sort that out after
    points.Add(FSegmentPoint{-segAB.Extent, a_idx});
    points.Add(FSegmentPoint{segAB.Extent, b_idx});
    // sort by T
    points.Sort([](const FSegmentPoint& pa, const FSegmentPoint& pb) { return pa.T < pb.T; });

    // connect sequential points, as long as they aren't the same point,
    // and the segment doesn't already exist
    for (int32 k = 0; k < points.Num() - 1; ++k)
    {
        int32 v0 = points[k].VID;
        int32 v1 = points[k + 1].VID;
        if (v0 == v1)
        {
            continue;
        }

        if (Graph.FindEdge(v0, v1) == FDynamicGraph2d::InvalidID)
        {
            // sanity check; technically this can happen and still be correct but it's more likely an error case
            ensureMsgf(FMath::Abs(points[k].T - points[k + 1].T) >= std::numeric_limits<float>::epsilon(), TEXT("insert_segment: different points have same T??"));

            const int32 EID = Graph.AppendEdge(v0, v1);
            Directions.Add(EID, v0 < v1);
        }
    }

    return true;
}

FIndex2i FText3DGlyphArrangement::SplitSegmentAtDistance(int32 InEID, double InDistance, double InTol)
{
    FIndex2i ev = Graph.GetEdgeV(InEID);
    FSegment2d seg = FSegment2d(Graph.GetVertex(ev.A), Graph.GetVertex(ev.B));

    int32 use_vid = -1;
    int32 new_eid = -1;
    if (InDistance < -(seg.Extent - InTol))
    {
        use_vid = ev.A;
    }
    else if (InDistance > (seg.Extent - InTol))
    {
        use_vid = ev.B;
    }
    else
    {
        FVector2d Pt = seg.PointAt(InDistance);
        FDynamicGraph2d::FEdgeSplitInfo splitInfo;
        EMeshResult result;
        int32 CrossingVert = FindExistingVertex(Pt);
        if (CrossingVert == -1)
        {
            result = Graph.SplitEdge(InEID, splitInfo);
        }
        else
        {
            result = Graph.SplitEdgeWithExistingVertex(InEID, CrossingVert, splitInfo);
        }
        ensureMsgf(result == EMeshResult::Ok, TEXT("insert_into_segment: edge split failed?"));
        use_vid = splitInfo.VNew;
        new_eid = splitInfo.ENewBN;

        Directions.Add(new_eid, !Directions[InEID]);

        if (CrossingVert == -1)
        {	// position + track added vertex
            Graph.SetVertex(use_vid, Pt);
            PointHash.InsertPoint(splitInfo.VNew, Pt);
        }
    }
    return FIndex2i(use_vid, new_eid);
}

int32 FText3DGlyphArrangement::FindExistingVertex(const FVector2d& InPoint) const
{
    return FindNearestVertex(InPoint, VertexSnapTol);
}

int32 FText3DGlyphArrangement::FindNearestVertex(const FVector2d& InPoint, double InSearchRadius, int32 InIgnoreVID) const
{
    auto FuncDistSq = [&](int32 B) { return DistanceSquared(InPoint, Graph.GetVertex(B)); };
    auto FuncIgnore = [&](int32 VID) { return VID == InIgnoreVID; };
    TPair<int32, double> found = (InIgnoreVID == -1) ? PointHash.FindNearestInRadius(InPoint, InSearchRadius, FuncDistSq)
                                                 : PointHash.FindNearestInRadius(InPoint, InSearchRadius, FuncDistSq, FuncIgnore);
    if (found.Key == PointHash.GetInvalidValue())
    {
        return -1;
    }
    return found.Key;
}

bool FText3DGlyphArrangement::FindIntersectingEdges(const FVector2d& InA, const FVector2d& InB, TArray<FIntersection>& OutHits, double InTol) const
{
    int32 num_hits = 0;
    FVector2d x = FVector2d::Zero(), y = FVector2d::Zero();
    FVector2d EPerp = UE::Geometry::PerpCW(InB - InA);
    UE::Geometry::Normalize(EPerp);
    for (int32 EID : Graph.EdgeIndices())
    {
        Graph.GetEdgeV(EID, x, y);
        // inlined version of WhichSide with pre-normalized EPerp, to ensure Tolerance is consistent for different edge lengths
        double SignX = EPerp.Dot(x - InA);
        double SignY = EPerp.Dot(y - InA);
        int32 SideX = (SignX > InTol ? +1 : (SignX < -InTol ? -1 : 0));
        int32 SideY = (SignY > InTol ? +1 : (SignY < -InTol ? -1 : 0));
        if (SideX == SideY && SideX != 0)
        {
            continue; // both pts on same side
        }

        FIntrSegment2Segment2d Intr(FSegment2d(x, y), FSegment2d(InA, InB));
        Intr.SetIntervalThreshold(InTol);
        // set a loose DotThreshold as well so almost-parallel segments are treated as parallel;
        //  otherwise we're more likely to hit later problems when an edge intersects near-overlapping edges at almost the same point
        // (TODO: detect + handle that case!)
        Intr.SetDotThreshold(1e-4);
        if (Intr.Find())
        {
            OutHits.Add(FIntersection{EID, SideX, SideY, Intr});
            num_hits++;
        }
    }

    return (num_hits > 0);
}

bool FText3DGlyphArrangement::FindIntersectingVertices(const FSegment2d &InSegmentAB, int32 InVIDA, int32 InVIDB, TArray<FSegmentPoint>& OutHits, double InTol)
{
    int32 num_hits = 0;

    for (int32 VID : Graph.VertexIndices())
    {
        if (Graph.GetVtxEdgeCount(VID) > 0 || VID == InVIDA || VID == InVIDB) // if it's an existing edge or on the currently added edge, it's not floating so skip it
        {
            continue;
        }

        FVector2d V = Graph.GetVertex(VID);
        double T;
        double DSQ = InSegmentAB.DistanceSquared(V, T);
        if (DSQ < InTol*InTol)
        {
            OutHits.Add(FSegmentPoint{ T, VID });
            num_hits++;
        }
    }

    return num_hits > 0;
}
