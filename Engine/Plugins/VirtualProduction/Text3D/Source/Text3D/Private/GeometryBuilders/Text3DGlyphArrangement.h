// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DPointHashGrid2d.h"
#include "Curve/DynamicGraph2.h"
#include "Intersection/IntrSegment2Segment2.h"

/**
 * @brief Modified copy of FArrangement2d, has edge directions
 */
struct FText3DGlyphArrangement
{
	using FIndex2i = UE::Geometry::FIndex2i;
	using FSegment2f = UE::Geometry::FSegment2f;
	using FSegment2d = UE::Geometry::FSegment2d;
	using FAxisAlignedBox2f = UE::Geometry::FAxisAlignedBox2f;

    UE::Geometry::FDynamicGraph2d Graph;
    UE::Text3D::FPointHashGrid2d PointHash;
    TMap<int32, bool> Directions; //bool - from A to B

	static constexpr double VertexSnapTol = 0.001;

	explicit FText3DGlyphArrangement(const FAxisAlignedBox2f& InBoundsHint);
    void Insert(const FSegment2f& Segment);

protected:
    struct FSegmentPoint
    {
        double T;
        int VID;
    };

    struct FIntersection
    {
        int EID;
        int SideX;
        int SideY;
        UE::Geometry::FIntrSegment2Segment2d Intr;
    };

    int32 InsertPoint(const FVector2d& InPoint, double InTol = 0);
    bool InsertSegment(FVector2d InA, FVector2d InB, double InTol = 0);
    UE::Geometry::FIndex2i SplitSegmentAtDistance(int InEID, double InDistance, double InTol);
    int32 FindExistingVertex(const FVector2d& InPoint) const;
    int32 FindNearestVertex(const FVector2d& InPoint, double InSearchRadius, int InIgnoreVID = -1) const;
    bool FindIntersectingEdges(const FVector2d& InA, const FVector2d& InB, TArray<FIntersection>& OutHits, double InTol = 0) const;
    bool FindIntersectingVertices(const FSegment2d& InSegmentAB, int32 InVIDA, int32 InVIDB, TArray<FSegmentPoint>& OutHits, double InTol = 0);
};
