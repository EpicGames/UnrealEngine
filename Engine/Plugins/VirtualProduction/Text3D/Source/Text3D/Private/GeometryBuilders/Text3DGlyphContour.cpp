// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphContour.h"

#include "GeometryBuilders/Text3DGlyphPart.h"

FText3DGlyphContour::~FText3DGlyphContour()
{
	for (FText3DGlyphPartPtr Part : *this)
	{
		Part->Prev.Reset();
		Part->Next.Reset();
	}
}

void FText3DGlyphContour::SetNeighbours()
{
	for (int32 Index = 0; Index < Num(); Index++)
	{
		FText3DGlyphPartPtr Point = (*this)[Index];

		Point->Prev = (*this)[(Index + Num() - 1) % Num()];
		Point->Next = (*this)[(Index + 1) % Num()];
	}
}

void FText3DGlyphContour::CopyFrom(const FText3DGlyphContour& Other)
{
	for (const FText3DGlyphPartConstPtr OtherPoint : Other)
	{
		Add(MakeShared<FText3DGlyphPart>(OtherPoint));
	}

	SetNeighbours();
}
