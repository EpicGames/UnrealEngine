// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "GeometryBuilders/Text3DGlyphPart.h"

class FText3DGlyphContour final : public TArray<FText3DGlyphPartPtr>
{
public:
	struct FPathEntry
	{
		int32 Prev;
		int32 Next;
	};

	FText3DGlyphContour() = default;
	~FText3DGlyphContour();

	int32 Find(const FText3DGlyphPartConstPtr Edge)
	{
		return TArray<FText3DGlyphPartPtr>::Find(ConstCastSharedPtr<FText3DGlyphPart>(Edge));
	}

	/**
	 * Set Prev and Next in parts.
	 */
	void SetNeighbours();
	/**
	 * Copy parts from other contour.
	 * @param Other - Controur from which parts should be copied.
	 */
	void CopyFrom(const FText3DGlyphContour& Other);
};
