// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphContourList.h"

#include "GeometryBuilders/Text3DGlyphData.h"
#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"

FText3DGlyphContourList::FText3DGlyphContourList(const FText3DGlyphContourList& Other)
{
	for (const FText3DGlyphContour& OtherContour : Other)
	{
		FText3DGlyphContour& Contour = Add();
		Contour.CopyFrom(OtherContour);
	}
}

void FText3DGlyphContourList::Initialize(FText3DGlyphData& Data)
{
	for (FText3DGlyphContour& Contour : *this)
	{
		Contour.SetNeighbours();

		for (FText3DGlyphPartPtr Edge : Contour)
		{
			Edge->ComputeTangentX();
		}

		for (const FText3DGlyphPartPtr& Point : Contour)
		{
			Point->ComputeSmooth();
		}

		for (int32 Index = 0; Index < Contour.Num(); Index++)
		{
			const FText3DGlyphPartPtr Point = Contour[Index];

			if (!Point->bSmooth && Point->TangentsDotProduct() > 0.f)
			{
				const FText3DGlyphPartPtr Curr = Point;
				const FText3DGlyphPartPtr Prev = Point->Prev;

				const float TangentsCrossProduct = FVector2D::CrossProduct(-Prev->TangentX, Curr->TangentX);
				const float MinTangentsCrossProduct = 0.9f;

				if (FMath::Abs(TangentsCrossProduct) < MinTangentsCrossProduct)
				{
					const float OffsetDefault = 0.01f;
					const float Offset = FMath::Min3(Prev->Length() / 2.f, Curr->Length() / 2.f, OffsetDefault);

					const FText3DGlyphPartPtr Added = MakeShared<FText3DGlyphPart>();
					Contour.Insert(Added, Index);

					Prev->Next = Added;
					Added->Prev = Prev;
					Added->Next = Curr;
					Curr->Prev = Added;

					const FVector2D CornerPosition = Curr->Position;

					Curr->Position = CornerPosition + Curr->TangentX * Offset;
					Added->Position = CornerPosition - Prev->TangentX * Offset;

					Data.AddVertices(1);
					const int32 VertexID = Data.AddVertex(Added->Position, { 1.f, 0.f }, { -1.f, 0.f, 0.f });

					Added->PathPrev.Add(VertexID);
					Added->PathNext.Add(VertexID);

					Added->ComputeTangentX();

					Added->ComputeSmooth();
					Curr->ComputeSmooth();
				}
			}
		}

		for (const FText3DGlyphPartPtr& Point : Contour)
		{
			Point->ComputeNormal();
			Point->ResetInitialPosition();
		}
	}
}

FText3DGlyphContour& FText3DGlyphContourList::Add()
{
	AddTail(FText3DGlyphContour());
	return GetTail()->GetValue();
}

void FText3DGlyphContourList::Remove(const FText3DGlyphContour& Contour)
{
	// Search with comparing pointers
	for (TDoubleLinkedList<FText3DGlyphContour>::TDoubleLinkedListNode* Node = GetHead(); Node; Node = Node->GetNextNode())
	{
		if (&Node->GetValue() == &Contour)
		{
			RemoveNode(Node);
			break;
		}
	}
}

void FText3DGlyphContourList::Reset()
{
	for (FText3DGlyphContour& Contour : *this)
	{
		for (const FText3DGlyphPartPtr& Part : Contour)
		{
			Part->ResetDoneExpand();
		}
	}
}
