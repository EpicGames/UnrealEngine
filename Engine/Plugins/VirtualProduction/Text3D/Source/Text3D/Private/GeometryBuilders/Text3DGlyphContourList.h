// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "GeometryBuilders/Text3DGlyphContour.h"
#include "Templates/SharedPointer.h"

class FText3DGlyphData;

class FText3DGlyphContourList final : public TDoubleLinkedList<FText3DGlyphContour>
{
public:
	FText3DGlyphContourList() = default;
	FText3DGlyphContourList(const FText3DGlyphContourList& Other);

	/**
	 * Initialize Countours.
	 * @param Data - Needed to add vertices for split corners.
	 */
	void Initialize(FText3DGlyphData& Data);

	/**
	 * Create contour.
	 * @return Reference to created contour.
	 */
	FText3DGlyphContour& Add();
	/**
	 * Remove contour.
	 * @param Contour - Const reference to contour that should be removed.
	 */
	void Remove(const FText3DGlyphContour& Contour);

	void Reset();
};
