// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Text3D/Private/Logs/Text3DLogs.h"
#include "Polygon2.h"
#include "Templates/SharedPointer.h"

using UE::Geometry::FPolygon2f;

struct FText3DGlyphContourNode;
using TText3DGlyphContourNodeShared = TSharedPtr<FText3DGlyphContourNode>;

struct FText3DGlyphContourNode final
{
	explicit FText3DGlyphContourNode(const TOptional<FPolygon2f> InContour, const bool bInCanHaveIntersections, const bool bInClockwise)
		: Contour(InContour)
		, bCanHaveIntersections(bInCanHaveIntersections)
		, bClockwise(bInClockwise)
	{
	}

	TOptional<FPolygon2f> Contour;

	// Needed for dividing contours with self-intersections: default value is true, false for parts of divided contours
	bool bCanHaveIntersections;

	bool bClockwise;

	// Contours that are inside this contour
	TArray<FText3DGlyphContourNode> Children;

	/** Debug print to display contour data */
	void Print(int32 InDepth = 0) const
	{
		const FString Indent = FString::ChrN(InDepth * 2, TEXT(' '));

		UE_LOGF(LogText3D, Log, "%lsContour: Vtx=%d, Area=%f, Bounds=%ls, CW=%ls",
			*Indent,
			Contour.IsSet() ? Contour->VertexCount() : 0,
			Contour.IsSet() ? Contour->SignedArea() : 0,
			Contour.IsSet() ? *Contour->Bounds().Extents().ToString() : TEXT(""),
			bClockwise ? TEXT("true") : TEXT("false"));

		for (const FText3DGlyphContourNode& Child : Children)
		{
			Child.Print(InDepth + 1);
		}
	}
};
