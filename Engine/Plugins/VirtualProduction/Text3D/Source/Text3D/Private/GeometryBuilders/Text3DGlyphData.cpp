// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphData.h"

#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Math/Vector2D.h"
#include "Math/Vector.h"
#include "Text3DModule.h"

FText3DGlyphData::FText3DGlyphData()
{
	CurrentGroup = EText3DGroupType::Front;
	GroupExpand = 0.0f;

	PlannedExtrude = 0.0f;
	PlannedExpand = 0.0f;

	NormalStart = {0.f, 0.f};
	NormalEnd = {0.f, 0.f};

	ExtrudeTarget = 0.0f;
	ExpandTarget = 0.0f;

	DoneExtrude = 0.0f;

	VertexCountBeforeAdd = 0;
	AddVertexIndex = 0;

	TriangleCountBeforeAdd = 0;
	AddTriangleIndex = 0;
}

void FText3DGlyphData::SetCurrentGroup(const EText3DGroupType Type, const float GroupExpandIn)
{
	CurrentGroup = Type;

	FText3DPolygonGroup& Group = Glyph.GetGroups()[static_cast<int32>(Type)];
	const FMeshDescription& MeshDescription = Glyph.GetMeshDescription();

	Group.FirstVertex = MeshDescription.Vertices().Num();
	Group.FirstTriangle = MeshDescription.Triangles().Num();

	GroupExpand = GroupExpandIn / UE::Text3D::Metrics::FontSizeInverse;
}

void FText3DGlyphData::PrepareSegment(const float PlannedExtrudeIn, const float PlannedExpandIn, const FVector2D NormalStartIn, const FVector2D NormalEndIn)
{
	PlannedExtrude = PlannedExtrudeIn;
	PlannedExpand = PlannedExpandIn / UE::Text3D::Metrics::FontSizeInverse;

	NormalStart = NormalStartIn;
	NormalEnd = NormalEndIn;
}

void FText3DGlyphData::SetTarget(const float ExtrudeTargetIn, const float ExpandTargetIn)
{
	ExtrudeTarget = ExtrudeTargetIn;
	ExpandTarget = ExpandTargetIn;
}

int32 FText3DGlyphData::AddVertices(const int32 Count)
{
	if (Count <= 0)
	{
		return 0;
	}

	FMeshDescription& MeshDescription = Glyph.GetMeshDescription();
	FStaticMeshAttributes& MeshAttributes = Glyph.GetStaticMeshAttributes();
	VertexCountBeforeAdd = MeshDescription.Vertices().Num();

	MeshDescription.ReserveNewVertices(Count);
	MeshDescription.ReserveNewVertexInstances(Count);

	for (int32 Index = 0; Index < Count; Index++)
	{
		const FVertexID Vertex = MeshDescription.CreateVertex();
		const FVertexInstanceID VertexInstance = MeshDescription.CreateVertexInstance(Vertex);
		MeshAttributes.GetVertexInstanceColors()[VertexInstance] = FVector4f(1.f, 1.f, 1.f, 1.f);
	}

	AddVertexIndex = 0;
	return VertexCountBeforeAdd;
}

int32 FText3DGlyphData::AddVertex(const FText3DGlyphPartConstPtr& Point, const FVector2D TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates)
{
	return AddVertex(Point->Position, TangentX, TangentZ, TextureCoordinates);
}

int32 FText3DGlyphData::AddVertex(const FVector2D Position, const FVector2D TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates)
{
	return AddVertex(GetVector(Position, DoneExtrude + ExtrudeTarget), {0.f, TangentX.X, TangentX.Y}, TangentZ, TextureCoordinates);
}

int32 FText3DGlyphData::AddVertex(const FVector& Position, const FVector& TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates)
{
	FStaticMeshAttributes& StaticMeshAttributes = Glyph.GetStaticMeshAttributes();
	const int32 VertexIndex = VertexCountBeforeAdd + AddVertexIndex++;
	StaticMeshAttributes.GetVertexPositions()[FVertexID(VertexIndex)] = (FVector3f)Position;
	const FVertexInstanceID Instance(static_cast<uint32>(VertexIndex));

	StaticMeshAttributes.GetVertexInstanceUVs()[Instance] = FVector2f(TextureCoordinates);
	StaticMeshAttributes.GetVertexInstanceNormals()[Instance] = (FVector3f)TangentZ;
	StaticMeshAttributes.GetVertexInstanceTangents()[Instance] = (FVector3f)TangentX;

	return VertexIndex;
}

void FText3DGlyphData::AddTriangles(const int32 Count)
{
	if (Count <= 0)
	{
		return;
	}

	FMeshDescription& MeshDescription = Glyph.GetMeshDescription();
	TriangleCountBeforeAdd = MeshDescription.Triangles().Num();
	MeshDescription.ReserveNewTriangles(Count);
	AddTriangleIndex = 0;
}

void FText3DGlyphData::AddTriangle(const int32 A, const int32 B, const int32 C)
{
	Glyph.GetMeshDescription().CreateTriangle(FPolygonGroupID(static_cast<int32>(CurrentGroup)), TArray<FVertexInstanceID>({FVertexInstanceID(A), FVertexInstanceID(B), FVertexInstanceID(C)}));
	AddTriangleIndex++;
}

float FText3DGlyphData::GetGroupExpand() const
{
	return GroupExpand;
}

float FText3DGlyphData::GetPlannedExtrude() const
{
	return PlannedExtrude;
}

float FText3DGlyphData::GetPlannedExpand() const
{
	return PlannedExpand;
}

void FText3DGlyphData::IncreaseDoneExtrude()
{
	DoneExtrude += PlannedExtrude;
}

FVector FText3DGlyphData::ComputeTangentZ(const FText3DGlyphPartConstPtr& Edge, const float DoneExpand)
{
	const FVector2D TangentX = Edge->TangentX;

	const float T = FMath::IsNearlyZero(PlannedExpand) ? 0.0f : DoneExpand / PlannedExpand;
	const FVector2D Normal = NormalStart * (1.f - T) + NormalEnd * T;

	const FVector2D TangentZ_YZ = FVector2D(TangentX.Y, -TangentX.X) * Normal.X;
	return FVector(Normal.Y, TangentZ_YZ.X, TangentZ_YZ.Y);
}

FVector2D FText3DGlyphData::Expanded(const FText3DGlyphPartConstPtr& Point) const
{
	// Needed expand value is difference of total expand and point's done expand
	return Point->Expanded(ExpandTarget - Point->DoneExpand);
}

void FText3DGlyphData::FillEdge(const FText3DGlyphPartPtr& Edge, const bool bSkipLastTriangle, bool bFlipNormals)
{
	const FText3DGlyphPartPtr EdgeA = Edge;
	const FText3DGlyphPartPtr EdgeB = Edge->Next;

	MakeTriangleFanAlongNormal(EdgeB, EdgeA, bFlipNormals, true);
	MakeTriangleFanAlongNormal(EdgeA, EdgeB, !bFlipNormals, false);

	if (!bSkipLastTriangle)
	{
		MakeTriangleFanAlongNormal(EdgeB, EdgeA, bFlipNormals, false);
	}
	else
	{
		// Index has to be removed despite last triangle being skipped.
		// For example when normals intersect and result of expansion of EdgeA and EdgeB is one point -
		// this point was already covered with last MakeTriangleFanAlongNormal call,
		// no need to keep it in neighbour point's path.
		EdgeA->PathNext.RemoveAt(0);
	}

	// Write done expand
	EdgeA->DoneExpand = ExpandTarget;
	EdgeB->DoneExpand = ExpandTarget;
}

FVector FText3DGlyphData::GetVector(const FVector2D Position, const float Height) const
{
	return (FVector(0.f, Position.X, Position.Y) * UE::Text3D::Metrics::FontSizeInverse + FVector(Height, 0.0f, 0.0f));
}

void FText3DGlyphData::MakeTriangleFanAlongNormal(const FText3DGlyphPartConstPtr& Cap, const FText3DGlyphPartPtr& Normal, const bool bNormalIsCapNext, const bool bSkipLastTriangle)
{
	TArray<int32>& Path = bNormalIsCapNext ? Normal->PathPrev : Normal->PathNext;
	const int32 Count = Path.Num() - (bSkipLastTriangle ? 2 : 1);

	// Create triangles
	AddTriangles(Count);

	for (int32 Index = 0; Index < Count; Index++)
	{
		AddTriangle((bNormalIsCapNext ? Cap->PathNext : Cap->PathPrev)[0],
						Path[bNormalIsCapNext ? Index + 1 : Index],
						Path[bNormalIsCapNext ? Index : Index + 1]);
	}

	// Remove covered vertices from path
	Path.RemoveAt(0, Count);
}
