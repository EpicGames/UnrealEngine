// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBuilders/Text3DGlyph.h"
#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Math/MathFwd.h"

/** Used to add vertices and triangles from different classes */
class FText3DGlyphData final
{
public:
	/**
	 * Constructor.
	 * @param InGlyph - Glyph that is being created.
	 */
	explicit FText3DGlyphData();

	const FText3DGlyph& GetGlyph() const
	{
		return Glyph;
	}

	FText3DGlyph& GetGlyph()
	{
		return Glyph;
	}

	/**
	 * Set group that is being created.
	 * @param Type - Group type.
	 * @param GroupExpandIn - Total expand of group (not needed now because intersections are not ready).
	 */
	void SetCurrentGroup(const EText3DGroupType Type, const float GroupExpandIn);
	/**
	 * Set data needed for bevelling one segment.
	 * @param PlannedExtrudeIn - Extrude.
	 * @param PlannedExpandIn - Expand.
	 * @param NormalStartIn - Normal in the start of segment.
	 * @param NormalEndIn - Normal in the end of segment.
	 */
	void PrepareSegment(const float PlannedExtrudeIn, const float PlannedExpandIn, const FVector2D NormalStartIn, const FVector2D NormalEndIn);
	/**
	 * Set data for single bevelling in segment (needed for intersections).
	 * @param ExtrudeTargetIn - Extrude.
	 * @param ExpandTargetIn - Expand.
	 */
	void SetTarget(const float ExtrudeTargetIn, const float ExpandTargetIn);

	int32 AddVertices(const int32 Count);
	int32 AddVertex(const FText3DGlyphPartConstPtr& Point, const FVector2D TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates = {0.f, 0.f});
	int32 AddVertex(const FVector2D Position, const FVector2D TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates = {0.f, 0.f});
	int32 AddVertex(const FVector& Position, const FVector& TangentX, const FVector& TangentZ, const FVector2D TextureCoordinates);

	void AddTriangles(const int32 Count);
	void AddTriangle(const int32 A, const int32 B, const int32 C);

	float GetGroupExpand() const;

	float GetPlannedExtrude() const;
	float GetPlannedExpand() const;

	/**
	 * Executed after segment was beveled.
	 */
	void IncreaseDoneExtrude();

	FVector ComputeTangentZ(const FText3DGlyphPartConstPtr& Edge, const float DoneExpand);


	/**
	 * FPart::Expanded for total expand value Data::ExpandTarget.
	 * @param Point - Point for which position should be computed.
	 * @return Computed position.
	 */
	FVector2D Expanded(const FText3DGlyphPartConstPtr& Point) const;

	/**
	 * Make triangulation of edge along paths of it's vertices (from end of previous triangulation to result of points' expansion). Removes covered points' indices from paths.
	 * @param Edge - Edge that has to be filled.
	 * @param bSkipLastTriangle - Do not create last triangle (furthest from end of previous triangulation).
	 * @param bFlipNormals - Reverse the geometry facing direction.
	 */
	void FillEdge(const FText3DGlyphPartPtr& Edge, const bool bSkipLastTriangle, bool bFlipNormals = false);

private:
	FText3DGlyph Glyph;

	EText3DGroupType CurrentGroup;
	float GroupExpand;

	float PlannedExtrude;
	float PlannedExpand;

	FVector2D NormalStart;
	FVector2D NormalEnd;

	float ExtrudeTarget;
	float ExpandTarget;

	float DoneExtrude;


	int32 VertexCountBeforeAdd;
	int32 AddVertexIndex;

	int32 TriangleCountBeforeAdd;
	int32 AddTriangleIndex;



	/**
	 * Transform position from glyph coordinate system to 3d.
	 * @param Position - Point position.
	 * @param Height - Offset in direction orthogonal to surface of front cap.
	 * @return 3d coordinate.
	 */
	FVector GetVector(const FVector2D Position, const float Height) const;
	/**
	 * Make triangle fan, called from FData::FillEdge.
	 * @param Cap - Cap of triangle fan.
	 * @param Normal - Point, fan will be created along it's normal.
	 * @param bNormalIsCapNext - Normal is next point after cap or vice versa.
	 * @param bSkipLastTriangle - See FData::FillEdge.
	 */
	void MakeTriangleFanAlongNormal(const FText3DGlyphPartConstPtr& Cap, const FText3DGlyphPartPtr& Normal, const bool bNormalIsCapNext, const bool bSkipLastTriangle);
};
