// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphLoader.h"

#include "BoxTypes.h"
#include "Curve/DynamicGraph2.h"
#include "GeometryBuilders/Text3DGlyphArrangement.h"
#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Polygon2.h"
#include "Text3DGlyphOutline.h"

using namespace UE::Geometry;

FText3DGlyphLoader::FText3DGlyphLoader()
	: EndIndex(-1)
	, VertexID(0)
{
}

#if WITH_FREETYPE
FText3DGlyphContourNode FText3DGlyphLoader::GenerateContourList(const UE::Text3D::FGlyphOutline& InGlyphOutline)
{
	FText3DGlyphContourNode Root(/*Contour*/{}, /*CanHaveIntersections*/false, /*Clockwise*/true);

	// @TODOFonts: The type for FT_Outline.Contours changed from short* to unsigned short* in FreeType 2.13.3
	// We need to still support both 2.10.0 and 2.14.1 for now so we used the introduced alias type
	// Fix after everything has been upgraded to 2.14.1
	// @see FGlyphOutline
	for (UE::Text3D::FFTOutlineContoursType ContourIndex : InGlyphOutline.Contours)
	{
		const TOptional<FPolygon2f> Contour = CreateContour(InGlyphOutline, ContourIndex);
		if (Contour.IsSet())
		{
			Insert(FText3DGlyphContourNode(Contour, /*CanHaveIntersections*/true, Contour->IsClockwise()), Root);
		}
	}

	if (!Root.Children.IsEmpty())
	{
		FixParityAndDivide(Root, false);
	}

	return Root;
}
#endif

FText3DGlyphLoader::FLine::FLine(const FVector2D PositionIn)
	: Position(PositionIn)
{
}

void FText3DGlyphLoader::FLine::Add(FText3DGlyphLoader* const Loader)
{
	if (!Loader->ProcessedContour.Num() || !(Position - Loader->FirstPosition).IsNearlyZero())
	{
		Loader->ProcessedContour.AddHead(FVector2f(Position));
	}
}

FText3DGlyphLoader::FCurve::FCurve(const bool bLineIn)
	: bLine(bLineIn)
	, StartT(0.f)
	, EndT(1.f)
{
	Loader = nullptr;

	Depth = 0.0f;
	MaxDepth = 0.0f;
	First = nullptr;
	Last = nullptr;
	bFirstSplit = false;
	bLastSplit = false;
}

FText3DGlyphLoader::FCurve::~FCurve()
{
}

struct FText3DGlyphLoader::FCurve::FPointData
{
	FPointData()
	{
		T = 0.0f;
		Point = nullptr;
	}

	float T;
	FVector2D Position;
	FVector2D Tangent;
	TDoubleLinkedList<FVector2f>::TDoubleLinkedListNode* Point;
};

void FText3DGlyphLoader::FCurve::Add(FText3DGlyphLoader* const LoaderIn)
{
	Loader = LoaderIn;

	if (bLine)
	{
		FLine(Position(StartT)).Add(Loader);
		return;
	}

	Depth = 0;

	FPointData Start;
	Start.T = StartT;
	Start.Position = Position(Start.T);
	Start.Tangent = Tangent(Start.T);
	Loader->ProcessedContour.AddHead(FVector2f(Start.Position));
	Start.Point = Loader->ProcessedContour.GetHead();
	First = Start.Point;
	bFirstSplit = false;

	FPointData End;
	End.T = EndT;
	End.Position = Position(End.T);
	End.Tangent = Tangent(End.T);
	End.Point = nullptr;
	Last = End.Point;
	bLastSplit = false;


	ComputeMaxDepth();
	Split(Start, End);
}

bool FText3DGlyphLoader::FCurve::OnOneLine(const FVector2D A, const FVector2D B, const FVector2D C)
{
	return FMath::IsNearlyZero(FVector2D::CrossProduct((B - A).GetSafeNormal(), (C - A).GetSafeNormal()));
}

void FText3DGlyphLoader::FCurve::ComputeMaxDepth()
{
	// Compute approximate curve length with 4 points
	const float MinStep = 30.f;
	const float StepT = 0.333f;
	float Length = 0.f;

	FVector2D Prev;
	FVector2D Current = Position(StartT);

	for (float T = StartT + StepT; T < EndT; T += StepT)
	{
		Prev = Current;
		Current = Position(T);

		Length += (Current - Prev).Size();
	}

	const float MaxStepCount = Length / MinStep;
	MaxDepth = static_cast<int32>(FMath::Log2(MaxStepCount)) + 1;
}

void FText3DGlyphLoader::FCurve::Split(const FPointData& Start, const FPointData& End)
{
	Depth++;
	FPointData Middle;

	Middle.T = (Start.T + End.T) / 2.f;
	Middle.Position = Position(Middle.T);
	Middle.Tangent = Tangent(Middle.T);
	Loader->ProcessedContour.InsertNode(FVector2f(Middle.Position), Start.Point);
	Middle.Point = Start.Point->GetPrevNode();

	CheckPart(Start, Middle);
	UpdateTangent(&Middle);
	CheckPart(Middle, End);
	Depth--;
}

void FText3DGlyphLoader::FCurve::CheckPart(const FPointData& Start, const FPointData& End)
{
	const FVector2D Side = (End.Position - Start.Position).GetSafeNormal();

	if ((FVector2D::DotProduct(Side, Start.Tangent) > FText3DGlyphPart::CosMaxAngleSideTangent &&
		FVector2D::DotProduct(Side, End.Tangent) > FText3DGlyphPart::CosMaxAngleSideTangent) || Depth >= MaxDepth)
	{
		if (!bFirstSplit && Start.Point == First)
		{
			bFirstSplit = true;
			Split(Start, End);
		}
		else if (!bLastSplit && End.Point == Last)
		{
			bLastSplit = true;
			Split(Start, End);
		}
	}
	else
	{
		Split(Start, End);
	}
}

void FText3DGlyphLoader::FCurve::UpdateTangent(FPointData* const Middle)
{
}

FText3DGlyphLoader::FQuadraticCurve::FQuadraticCurve(const FVector2D A, const FVector2D B, const FVector2D C)
	: FCurve(OnOneLine(A, B, C))

	, E(A - 2.f * B + C)
	, F(-A + B)
	, G(A)
{
}

FVector2D FText3DGlyphLoader::FQuadraticCurve::Position(const float T) const
{
	return E * T * T + 2.f * F * T + G;
}

FVector2D QuadraticCurveTangent(const FVector2D E, const FVector2D F, const float T)
{
	const FVector2D Result = E * T + F;

	if (Result.IsNearlyZero())
	{
		// Just some vector with non-zero length
		return {1.f, 0.f};
	}

	return Result;
}

FVector2D FText3DGlyphLoader::FQuadraticCurve::Tangent(const float T)
{
	return QuadraticCurveTangent(E, F, T).GetSafeNormal();
}

FText3DGlyphLoader::FCubicCurve::FCubicCurve(const FVector2D A, const FVector2D B, const FVector2D C, const FVector2D D)
	: FCurve(OnOneLine(A, B, C) && OnOneLine(B, C, D))
	, E(-A + 3.f * B - 3.f * C + D)
	, F(A - 2.f * B + C)
	, G(-A + B)
	, H(A)
	, bSharpStart(false)
	, bSharpMiddle(false)
	, bSharpEnd(false)
{
	if (!bLine)
	{
		bSharpStart = G.IsNearlyZero();
		bSharpEnd = (C - D).IsNearlyZero();
	}
}

void FText3DGlyphLoader::FCubicCurve::UpdateTangent(FPointData* const Middle)
{
	// In this point curve is not smooth, and  r'(t + 0) / |r'(t + 0)| = -r'(t - 0) / |r'(t - 0)|
	if (bSharpMiddle)
	{
		bSharpMiddle = false;
		Middle->Tangent *= -1.f;
	}
}

FVector2D FText3DGlyphLoader::FCubicCurve::Position(const float T) const
{
	return E * T * T * T + 3.f * F * T * T + 3.f * G * T + H;
}

FVector2D FText3DGlyphLoader::FCubicCurve::Tangent(const float T)
{
	FVector2D Result;

	// Using  r' / |r'|  for sharp start and end
	if (bSharpStart && FMath::IsNearlyEqual(T, StartT))
	{
		Result = F;
	}
	else if (bSharpEnd && FMath::IsNearlyEqual(T, EndT))
	{
		Result = -(E + F);
	}
	else
	{
		Result = E * T * T + 2.f * F * T + G;
		bSharpMiddle = Result.IsNearlyZero();

		if (bSharpMiddle)
		{
			// Using derivative of quadratic bezier curve (A, B, C) in this point
			Result = QuadraticCurveTangent(F, G, T);
		}
	}

	return Result.GetSafeNormal();
}

#if WITH_FREETYPE
TOptional<FPolygon2f> FText3DGlyphLoader::CreateContour(const UE::Text3D::FGlyphOutline& InGlyphOutline, int32 InContourIndex)
{
	TOptional<FPolygon2f> Contour = ProcessFreetypeOutline(InGlyphOutline, InContourIndex);
	if (Contour.IsSet() && RemoveBadPoints(*Contour))
	{
		return Contour;
	}
	return {};
}

TOptional<FPolygon2f> FText3DGlyphLoader::ProcessFreetypeOutline(const UE::Text3D::FGlyphOutline& InGlyphOutline, int32 InContourIndex)
{
	const int32 StartIndex = EndIndex + 1;
	EndIndex = InContourIndex;
	const int32 ContourLength = EndIndex - StartIndex + 1;

	if (ContourLength < 3)
	{
		return {};
	}

	ProcessedContour.Empty();

	const FT_Vector* const Points = InGlyphOutline.Points.GetData() + StartIndex;
	auto ToFVector2D = [Points](const int32 Index)
	{
		const FT_Vector Point = Points[Index];
		return FVector2D(Point.x, Point.y);
	};

	FVector2D Prev;
	FVector2D Current = ToFVector2D(ContourLength - 1);
	FVector2D Next = ToFVector2D(0);
	FVector2D NextNext = ToFVector2D(1);

	// @TODOFonts: There has been a change to the type of FT_Outline.Tags from char* to unsigned char* in FreeType2-2.13.3
	// We want to support both 2.10.0 and 2.14.1 for now so we use the introduced alias for now.
	// @see FText3DGlyphOutline
	const UE::Text3D::FFTOutlineTagsType* const Tags = InGlyphOutline.Tags.GetData() + StartIndex;
	auto Tag = [Tags](int32 Index)
	{
		return FT_CURVE_TAG(Tags[Index]);
	};

	int32 TagPrev = 0;
	int32 TagCurrent = Tag(ContourLength - 1);
	int32 TagNext = Tag(0);


	FVector2D& FirstPositionLocal = FirstPosition;
	TDoubleLinkedList<FVector2f>& ProcessedContourLocal = ProcessedContour;
	auto ContourIsBad = [&FirstPositionLocal, &ProcessedContourLocal](const FVector2D Point)
	{
		if (ProcessedContourLocal.Num() == 0)
		{
			FirstPositionLocal = Point;
			return false;
		}

		return (Point - FVector2D(ProcessedContourLocal.GetHead()->GetValue())).IsNearlyZero();
	};

	for (int32 Index = 0; Index < ContourLength; Index++)
	{
		const int32 NextIndex = (Index + 1) % ContourLength;

		Prev = Current;
		Current = Next;
		Next = NextNext;
		NextNext = ToFVector2D((NextIndex + 1) % ContourLength);

		TagPrev = TagCurrent;
		TagCurrent = TagNext;
		TagNext = Tag(NextIndex);

		if (TagCurrent == FT_Curve_Tag_On)
		{
			if (TagNext == FT_Curve_Tag_Cubic || TagNext == FT_Curve_Tag_Conic)
			{
				continue;
			}

			if (ContourIsBad(Current))
			{
				return {};
			}

			if (TagNext == FT_Curve_Tag_On && (Current - Next).IsNearlyZero())
			{
				continue;
			}

			FLine(Current).Add(this);
		}
		else if (TagCurrent == FT_Curve_Tag_Conic)
		{
			FVector2D A;

			if (TagPrev == FT_Curve_Tag_On)
			{
				if (ContourIsBad(Prev))
				{
					return {};
				}

				A = Prev;
			}
			else
			{
				A = (Prev + Current) / 2.f;
			}

			FQuadraticCurve(A, Current, TagNext == FT_Curve_Tag_Conic ? (Current + Next) / 2.f : Next).Add(this);
		}
		else if (TagCurrent == FT_Curve_Tag_Cubic && TagNext == FT_Curve_Tag_Cubic)
		{
			if (ContourIsBad(Prev))
			{
				return {};
			}

			FCubicCurve(Prev, Current, Next, NextNext).Add(this);
		}
	}

	if (ProcessedContour.Num() < 3)
	{
		return {};
	}

	FPolygon2f Contour;
	for (const TDoubleLinkedList<FVector2f>::TDoubleLinkedListNode* Point = ProcessedContour.GetTail(); Point; Point = Point->GetPrevNode())
	{
		Contour.AppendVertex(Point->GetValue());
	}
	return Contour;
}
#endif

bool FText3DGlyphLoader::IsInside(const FPolygon2f& ContourA, const FPolygon2f& ContourB) const
{
	return ContourB.Contains(ContourA);
}

void FText3DGlyphLoader::Insert(FText3DGlyphContourNode&& NodeA, FText3DGlyphContourNode& NodeB) const
{
	for (int32 ChildBIndex = 0; ChildBIndex < NodeB.Children.Num(); ChildBIndex++)
	{
		FText3DGlyphContourNode& ChildB = NodeB.Children[ChildBIndex];

		if (IsInside(*NodeA.Contour, *ChildB.Contour))
		{
			Insert(MoveTemp(NodeA), ChildB);
			return;
		}

		if (IsInside(*ChildB.Contour, *NodeA.Contour))
		{
			// add ChildBContour to list of contours that are inside ContourA
			NodeA.Children.Add(ChildB);
			// replace ChildBContour with ContourA in list it was before
			ChildB = NodeA;

			// check if other contours in that list are inside ContourA
			for (int32 ChildBSiblingIndex = NodeB.Children.Num() - 1; ChildBSiblingIndex > ChildBIndex; ChildBSiblingIndex--)
			{
				FText3DGlyphContourNode& ChildBSibling = NodeB.Children[ChildBSiblingIndex];
				if (IsInside(*ChildBSibling.Contour, *NodeA.Contour))
				{
					NodeA.Children.Add(ChildBSibling);
					NodeB.Children.RemoveAt(ChildBSiblingIndex);
				}
			}

			return;
		}
	}

	NodeB.Children.Add(MoveTemp(NodeA));
}

void FText3DGlyphLoader::FixParityAndDivide(FText3DGlyphContourNode& Node, const bool bInClockwise)
{
	for (int32 ChildIndex = 0; ChildIndex < Node.Children.Num(); ChildIndex++)
	{
		FText3DGlyphContourNode& Child = Node.Children[ChildIndex];

		// bCanHaveIntersections has default value "true", if it's false then contour was checked for having self-intersections and it's parity was fixed
		if (!Child.bCanHaveIntersections)
		{
			FixParityAndDivide(Child, !bInClockwise);
			continue;
		}

		// Fix parity
		if (Child.bClockwise != bInClockwise)
		{
			Child.Contour->Reverse();
			Child.bClockwise = bInClockwise;
		}

		FText3DGlyphArrangement Arrangement = MakeArrangement(*Child.Contour);
		Junctions.Reset();

		// Find first junction
		if (!FindJunction(Arrangement))
		{
			// If no junction was found, contour doesn't have self-intersections
			FixParityAndDivide(Child, !bInClockwise);
			continue;
		}

		// Remove this contour from parent's child list
		FText3DGlyphContourNode RemovedChildNode = MoveTemp(Child);
		Node.Children.RemoveAt(ChildIndex--);

		// Reset path
		DividedContourIDs.Reset();
		// Parts are stored as children of separate root node
		FText3DGlyphContourNode RootForDetaching({}, /*CanHaveIntersections*/false, Node.bClockwise);
		// Processed edges are removed from graph to simplify dividing, vertices that don't belong to any edge are removed too, so a copy is needed
		const TArray<FVector2f> InitialVertices = CopyVertices(Arrangement);

		bool bAddPoint = true;
		while (bAddPoint)
		{
			AddPoint(Arrangement);

			if (VertexID == -1)
			{
				// This is bad contour (not a closed loop)
				VertexID = Junctions.Last();
				DetachBadContour();
			}
			else
			{
				const int32 RepeatedJunctionIndex = Junctions.Find(VertexID);
				if (RepeatedJunctionIndex == INDEX_NONE)
				{
					continue;
				}

				// If it is, we made a loop that should be separated
				DetachFinishedContour(RepeatedJunctionIndex, InitialVertices, RootForDetaching);
			}

			// If path contains no junctions and current vertex is not a junction
			if (Junctions.Num() == 0 && !Arrangement.Graph.IsJunctionVertex(VertexID))
			{
				if (!FindJunction(Arrangement))
				{
					// If there are no junctions in graph, it contains last contour that should be separated
					if (!FindRegular(Arrangement))
					{
						// Graph is empty, done
						bAddPoint = false;
					}
					else
					{
						// Add non-junction vertex as if it was a junction
						AddPoint(Arrangement, true);
					}
				}
			}
		}

		RemoveUnneededNodes(RootForDetaching);
		MergeRootForDetaching(MoveTemp(RemovedChildNode), RootForDetaching, Node);
	}
}

bool FText3DGlyphLoader::IsBadNormal(const FPolygon2f& InContour, const int32 InPoint) const
{
	FVector2f ToPrev;
	FVector2f ToNext;
	InContour.NeighbourVectors(InPoint, ToPrev, ToNext, /*Normalize*/true);
	return FMath::IsNearlyZero(1.0f - FVector2D::DotProduct(FVector2D(ToPrev), FVector2D(ToNext)));
}

bool FText3DGlyphLoader::MergedWithNext(const FPolygon2f& InContour, const int32 Point) const
{
	const int32 Current = Point;
	const int32 Next = (Point + 1) % InContour.VertexCount();
	return FVector2D(InContour[Next] - InContour[Current]).IsNearlyZero();
}

bool FText3DGlyphLoader::RemoveBadPoints(FPolygon2f& InOutContour) const
{
	int32 VertexCount = InOutContour.VertexCount();
	int32 CheckVertexIndex = 0;
	int32 ConsecutiveCleanVertices = 0;

	while (ConsecutiveCleanVertices < VertexCount)
	{
		if (IsBadNormal(InOutContour, CheckVertexIndex) || MergedWithNext(InOutContour, CheckVertexIndex))
		{
			InOutContour.RemoveVertex(CheckVertexIndex);

			// Update count since we removed one
			VertexCount = InOutContour.VertexCount();
			if (VertexCount < 4)
			{
				return false;
			}

			// After removal, the next vertex now occupies CheckVertexIndex
			// Wrap around if we removed the last vertex
			CheckVertexIndex = CheckVertexIndex % VertexCount;

			// Reset counter since we made a change
			ConsecutiveCleanVertices = 0;
		}
		else
		{
			// This vertex was clean, move to next
			ConsecutiveCleanVertices++;
			CheckVertexIndex = (CheckVertexIndex + 1) % VertexCount;
		}
	}
	return true;
}

FText3DGlyphArrangement FText3DGlyphLoader::MakeArrangement(const FPolygon2f& InContour)
{
	FAxisAlignedBox2f Box = FAxisAlignedBox2f::Empty();
	for (const FVector2f& Vertex : InContour.GetVertices())
	{
		Box.Contain(Vertex);
	}

	FText3DGlyphArrangement Arrangement(Box);
	for (FSegment2f Edge : InContour.Segments())
	{
		Arrangement.Insert(Edge);
	}
	return Arrangement;
}

bool FText3DGlyphLoader::FindJunction(const FText3DGlyphArrangement& InArrangement)
{
	for (int32 VID = 0; VID < InArrangement.Graph.MaxVertexID(); VID++)
	{
		if (InArrangement.Graph.IsJunctionVertex(VID))
		{
			VertexID = VID;
			return true;
		}
	}
	VertexID = -1;
	return false;
}

bool FText3DGlyphLoader::FindRegular(const FText3DGlyphArrangement& InArrangement)
{
	for (int32 VID = 0; VID < InArrangement.Graph.MaxVertexID(); VID++)
	{
		if (InArrangement.Graph.IsRegularVertex(VID))
		{
			VertexID = VID;
			return true;
		}
	}

	VertexID = -1;
	return false;
}

TArray<FVector2f> FText3DGlyphLoader::CopyVertices(const FText3DGlyphArrangement& InArrangement) const
{
	TArray<FVector2f> Vertices;
	const int32 MaxVID = InArrangement.Graph.MaxVertexID();
	Vertices.Reserve(MaxVID);

	for (int32 VID = 0; VID < MaxVID; VID++)
	{
		const FVector2d Vertex = InArrangement.Graph.GetVertex(VID);
		Vertices.Add({static_cast<float>(Vertex.X), static_cast<float>(Vertex.Y)});
	}

	return Vertices;
}

void FText3DGlyphLoader::DetachBadContour()
{
	const int32 JunctionIndexInContour = FindStartOfDetachedContour(Junctions.Num() - 1);
	RemoveDetachedContourFromPath(JunctionIndexInContour);
}

void FText3DGlyphLoader::DetachFinishedContour(const int32 RepeatedJunctionIndex, const TArray<FVector2f>& InitialVertices, FText3DGlyphContourNode& RootForDetaching)
{
	const int32 JunctionIndexInContour = FindStartOfDetachedContour(RepeatedJunctionIndex);

	// Create contour from copied vertices
	FPolygon2f FinishedContour;
	for (int32 ID = JunctionIndexInContour; ID < DividedContourIDs.Num(); ID++)
	{
		FinishedContour.AppendVertex(InitialVertices[DividedContourIDs[ID]]);
	}

	RemoveDetachedContourFromPath(JunctionIndexInContour);
	Insert(FText3DGlyphContourNode(FinishedContour, /*CanHaveIntersections*/false, FinishedContour.IsClockwise()), RootForDetaching);
}

int32 FText3DGlyphLoader::FindStartOfDetachedContour(const int32 RepeatedJunctionIndex)
{
	// Remove from list of junctions all junctions that belong to detached loop
	Junctions.SetNum(RepeatedJunctionIndex);
	// Find index in path of first (and last) vertex of loop
	const int32 JunctionIndexInContour = DividedContourIDs.Find(VertexID);
	return JunctionIndexInContour;
}

void FText3DGlyphLoader::RemoveDetachedContourFromPath(const int32 JunctionIndexInContour)
{
	DividedContourIDs.SetNum(JunctionIndexInContour);
}

void FText3DGlyphLoader::AddPoint(FText3DGlyphArrangement& InArrangement, const bool bForceJunction)
{
	DividedContourIDs.Add(VertexID);
	int32 OutgoingEdge = 0;

	if (InArrangement.Graph.IsJunctionVertex(VertexID) || bForceJunction)
	{
		Junctions.Add(VertexID);
		// Select specific path direction
		OutgoingEdge = FindOutgoing(InArrangement, Junctions.Last());
	}
	else if (InArrangement.Graph.IsVertex(VertexID))
	{
		// This should be the only possible direction
		OutgoingEdge = *InArrangement.Graph.VtxEdgesItr(VertexID).begin();
	}
	else
	{
		// We have no possible directions, this contour is bad
		VertexID = -1;
		return;
	}

	// Move to next graph node
	VertexID = GetOtherEdgeVertex(InArrangement, OutgoingEdge, VertexID);
	InArrangement.Graph.RemoveEdge(OutgoingEdge, true);
}

void FText3DGlyphLoader::RemoveUnneededNodes(FText3DGlyphContourNode& Node) const
{
	for (int32 ChildIndex = 0; ChildIndex < Node.Children.Num();)
	{
		FText3DGlyphContourNode& Child = Node.Children[ChildIndex];
		if (Child.bClockwise == Node.bClockwise)
		{
			Node.Children.RemoveAt(ChildIndex);
		}
		else
		{
			ChildIndex++;
			RemoveUnneededNodes(Child);
		}
	}
}

void FText3DGlyphLoader::MergeRootForDetaching(FText3DGlyphContourNode&& RemovedNode, FText3DGlyphContourNode& RootForDetaching, FText3DGlyphContourNode& RemovedNodeParent) const
{
	for (FText3DGlyphContourNode& Child : RemovedNode.Children)
	{
		Insert(MoveTemp(Child), RootForDetaching);
	}
	RemovedNode.Children.Reset();

	for (FText3DGlyphContourNode& Child : RootForDetaching.Children)
	{
		RemovedNodeParent.Children.Add(Child);
	}
}

bool FText3DGlyphLoader::IsOutgoing(const FText3DGlyphArrangement& InArrangement, const int32 Junction, const int32 EID) const
{
	return (InArrangement.Graph.GetEdgeV(EID).A == Junction) == InArrangement.Directions[EID];
}

int32 FText3DGlyphLoader::FindOutgoing(const FText3DGlyphArrangement& InArrangement, const int32 Junction) const
{
	// Get list of all edges that have this vertex
	TArray<int32> Edges;
	SortedVtxEdges(InArrangement, Junction, Edges);

	int32 Current = 0;
	int32 Next = Edges[0];

	for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); EdgeIndex++)
	{
		Current = Next;
		Next = Edges[(EdgeIndex + 1) % Edges.Num()];

		// Needed direction is the one that is outgoing and it's clockwise neighbour is not
		if (!IsOutgoing(InArrangement, Junction, Current) && IsOutgoing(InArrangement, Junction, Next))
		{
			return Next;
		}
	}

	return Edges[0];
}

int32 FText3DGlyphLoader::GetOtherEdgeVertex(const FText3DGlyphArrangement& InArrangement, const int32 EID, const int32 VID) const
{
	const FIndex2i Edge = InArrangement.Graph.GetEdgeV(EID);
	return (Edge.A == VID) ? Edge.B : ((Edge.B == VID) ? Edge.A : FDynamicGraph::InvalidID);
}

void FText3DGlyphLoader::SortedVtxEdges(const FText3DGlyphArrangement& InArrangement, const int32 VID, TArray<int32>& Sorted) const
{
	Sorted.Reserve(InArrangement.Graph.GetVtxEdgeCount(VID));

	for (int32 EID : InArrangement.Graph.VtxEdgesItr(VID))
	{
		Sorted.Add(EID);
	}

	const FVector2d V = InArrangement.Graph.GetVertex(VID);
	Algo::SortBy(Sorted,
		[&](int32 EID)
		{
			 const int32 NbrVID = GetOtherEdgeVertex(InArrangement, EID, VID);
			 const FVector2d D = InArrangement.Graph.GetVertex(NbrVID) - V;
			 return TMathUtil<double>::Atan2Positive(D.Y, D.X);
		});
}
