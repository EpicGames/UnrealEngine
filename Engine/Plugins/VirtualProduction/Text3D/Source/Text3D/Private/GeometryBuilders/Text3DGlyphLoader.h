// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/List.h"
#include "GeometryBuilders/Text3DGlyphContourNode.h"
#include "Math/Vector2D.h"
#include "Polygon2.h"
#include "Templates/SharedPointer.h"
#include "Text3DModule.h"

struct FText3DGlyphArrangement;
using UE::Geometry::FPolygon2f;

namespace UE::Text3D
{
	struct FGlyphOutline;
}

/** Creates glyph geometry data from a font glyphs */
class FText3DGlyphLoader final
{
public:
	explicit FText3DGlyphLoader();

#if WITH_FREETYPE
	FText3DGlyphContourNode GenerateContourList(const UE::Text3D::FGlyphOutline& InGlyphOutline);
#endif

private:
	class FLine final
	{
	public:
		FLine(const FVector2D PositionIn);
		void Add(FText3DGlyphLoader* const Loader);

	private:
		const FVector2D Position;
	};

	class FCurve
	{
	public:
		FCurve(const bool bLineIn);
		virtual ~FCurve();

		void Add(FText3DGlyphLoader* const LoaderIn);

	protected:
		struct FPointData;


		/**
		 * Check if 3 points are on one line.
		 * @param A - Point A.
		 * @param B - Point B.
		 * @param C - Point C.
		 * @return Are points on one line?
		 */
		static bool OnOneLine(const FVector2D A, const FVector2D B, const FVector2D C);


		/** Is curve a line? */
		const bool bLine;
		FText3DGlyphLoader* Loader;

		const float StartT;
		const float EndT;

	private:
		int32 Depth;
		int32 MaxDepth;

		TDoubleLinkedList<FVector2f>::TDoubleLinkedListNode* First;
		TDoubleLinkedList<FVector2f>::TDoubleLinkedListNode* Last;

		/** Needed to make additional splits near start and end of curve */
		bool bFirstSplit;
		bool bLastSplit;


		/**
		 * Compute max depth (depends on curve length, step is fixed).
		 */
		void ComputeMaxDepth();
		void Split(const FPointData& Start, const FPointData& End);
		void CheckPart(const FPointData& Start, const FPointData& End);
		virtual void UpdateTangent(FPointData* const Middle);

		virtual FVector2D Position(const float T) const = 0;
		virtual FVector2D Tangent(const float T) = 0;
	};

	class FQuadraticCurve final : public FCurve
	{
	public:
		FQuadraticCurve(const FVector2D A, const FVector2D B, const FVector2D C);

	private:
		const FVector2D E;
		const FVector2D F;
		const FVector2D G;


		FVector2D Position(const float T) const override;
		FVector2D Tangent(const float T) override;
	};

	class FCubicCurve final : public FCurve
	{
	public:
		FCubicCurve(const FVector2D A, const FVector2D B, const FVector2D C, const FVector2D D);

	private:
		const FVector2D E;
		const FVector2D F;
		const FVector2D G;
		const FVector2D H;

		/** Sharp means that curve derivative has zero length, it's actually sharp only in middle case */
		bool bSharpStart;
		bool bSharpMiddle;
		bool bSharpEnd;


		void UpdateTangent(FPointData* const Middle) override;

		FVector2D Position(const float t) const override;
		FVector2D Tangent(const float t) override;
	};

	int32 EndIndex;
	TDoubleLinkedList<FVector2f> ProcessedContour;
	FVector2D FirstPosition;
	// List of junctions in path
	TArray<int32> Junctions;
	// Current vertex that should be added to path
	int32 VertexID;
	// Path
	TArray<int32> DividedContourIDs;

#if WITH_FREETYPE
	TOptional<FPolygon2f> CreateContour(const UE::Text3D::FGlyphOutline& InGlyphOutline, int32 InContourIndex);
	TOptional<FPolygon2f> ProcessFreetypeOutline(const UE::Text3D::FGlyphOutline& InGlyphOutline, int32 InContourIndex);
#endif

	/**
	 * Checks if ContourA is inside ContourB.
	 * @param ContourA
	 * @param ContourB
	 */
	bool IsInside(const FPolygon2f& ContourA, const FPolygon2f& ContourB) const;

	/**
	 * Insert NodeA inside NodeB.
	 * @param NodeA
	 * @param NodeB
	 */
	void Insert(FText3DGlyphContourNode&& NodeA, FText3DGlyphContourNode& NodeB) const;
	/**
	 * Reverse contour if it's initial parity differs from the one it should have, find self-intersections and, if found, divide to contours without self-intersections.
	 * @param Node - Function is called recursively to fix all contours inside Node->Contour.
	 * @param bInClockwise - The parity that contours listed in Node->Nodes should have.
	 */
	void FixParityAndDivide(FText3DGlyphContourNode& Node, const bool bInClockwise);

	bool RemoveBadPoints(FPolygon2f& InOutContour) const;

	bool IsBadNormal(const FPolygon2f& InContour, const int32 InPoint) const;
	bool MergedWithNext(const FPolygon2f& InContour, const int32 Point) const;

	FText3DGlyphArrangement MakeArrangement(const FPolygon2f& InContour);
	/**
	 * Find vertex that belongs to more then 2 edges and write it's ID to VertexID member.
	 * @return Was such vertex found?
	 */
	bool FindJunction(const FText3DGlyphArrangement& InArrangement);
	/**
	 * Find vertex that belongs to 2 edges and write it's ID to VertexID member.
	 * @return Was such vertex found?
	 */
	bool FindRegular(const FText3DGlyphArrangement& InArrangement);
	TArray<FVector2f> CopyVertices(const FText3DGlyphArrangement& InArrangement) const;

	/**
	 * Separate bad contour (the one that is not a closed loop) from path.
	 */
	void DetachBadContour();
	/**
	 * Separate closed loop from path.
	 * @param RepeatedJunctionIndex - Index of start (and end) of loop in list of junctions.
	 * @param InitialVertices - Previously copied graph vertices.
	 * @param RootForDetaching - Separate root node for detaching.
	 */
	void DetachFinishedContour(const int32 RepeatedJunctionIndex, const TArray<FVector2f>& InitialVertices, FText3DGlyphContourNode& RootForDetaching);

	/**
	 * Find start of contour that is detached, remove it from list of junctions.
	 * @param RepeatedJunctionIndex - Index of start (and end) of loop in list of junctions.
	 * @return Index in path of first (and last) vertex of loop.
	 */
	int32 FindStartOfDetachedContour(const int32 RepeatedJunctionIndex);
	/**
	 * Remove separated contour from path.
	 * @param JunctionIndexInContour - Index in path of first (and last) vertex of loop.
	 */
	void RemoveDetachedContourFromPath(const int32 JunctionIndexInContour);

	/**
	 * Add vertex to path
	 * @param InArrangement the arrangement to query
	 * @param bForceJunction - Needed for separating last loop, it has no junctions so any other vertex has to be treated as one.
	 */
	void AddPoint(FText3DGlyphArrangement& InArrangement, const bool bForceJunction = false);

	/**
	 * Check what nodes have same parity as their parents and remove them, they cover the area that already is covered by parent.
	 * @param Node - Processed node.
	 */
	void RemoveUnneededNodes(FText3DGlyphContourNode& Node) const;

	/**
	 * Merge separate root to regular one.
	 * @param RemovedNode - Contour that was divided.
	 * @param RootForDetaching - Separate root that stores result of division.
	 * @param RemovedNodeParent - Parent of contour that was divided.
	 */
	void MergeRootForDetaching(FText3DGlyphContourNode&& RemovedNode, FText3DGlyphContourNode& RootForDetaching, FText3DGlyphContourNode& RemovedNodeParent) const;

	/**
	 * Choose next path direction in junction vertex.
	 * @param Junction - Vertex ID of junction.
	 * @return Edge ID of chosen direction.
	 */
	int32 FindOutgoing(const FText3DGlyphArrangement& InArrangement, const int32 Junction) const;
	bool IsOutgoing(const FText3DGlyphArrangement& InArrangement, const int32 Junction, const int32 EID) const;

	// Copied from FDynamicGraph2d because it's private there
	int32 GetOtherEdgeVertex(const FText3DGlyphArrangement& InArrangement, const int32 EID, const int32 VID) const;

	// Fixed version of FDynamicGraph2d::SortedVtxEdges
	void SortedVtxEdges(const FText3DGlyphArrangement& InArrangement, const int32 VID, TArray<int32>& Sorted) const;
};
