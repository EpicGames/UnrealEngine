// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphMeshBuilder.h"

#include "ConstrainedDelaunay2.h"
#include "GeometryBuilders/Text3DGlyphContour.h"
#include "GeometryBuilders/Text3DGlyphContourList.h"
#include "GeometryBuilders/Text3DGlyphData.h"
#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Subsystems/Text3DEngineSubsystem.h"

using namespace UE::Geometry;

void FText3DGlyphMeshBuilder::CreateMeshes(const FText3DGlyphContourNode& InRoot, const float InExtrude, const float InBevel, const EText3DBevelType InBevelType, const int32 InBevelSegments, const bool bInOutline, const float InOutlineExpand, const EText3DOutlineType InOutlineType)
{
	CreateFrontMesh(InRoot, bInOutline, InOutlineExpand, InOutlineType);

	if (Contours.Num() == 0)
	{
		return;
	}

	const bool bFlipNormals = FMath::Sign(Data.GetPlannedExpand()) < 0;
	const float BevelLocal = bInOutline ? 0.f : InBevel;

	if (!bInOutline)
	{
		CreateBevelMesh(BevelLocal, InBevelType, InBevelSegments);	
	}

	CreateExtrudeMesh(InExtrude, BevelLocal, InBevelType, bFlipNormals);
}

void FText3DGlyphMeshBuilder::SetFrontAndBevelTextureCoordinates(const float Bevel)
{
	EText3DGroupType GroupType = FMath::IsNearlyZero(Bevel) ? EText3DGroupType::Front : EText3DGroupType::Bevel;
	int32 GroupIndex = static_cast<int32>(GroupType);

	FBox2f Box;
	TText3DGroupList& Groups = Data.GetGlyph().GetGroups();

	const int32 FirstVertex = Groups[GroupIndex].FirstVertex;
	const int32 LastVertex = Groups[GroupIndex + 1].FirstVertex;

	TVertexAttributesConstRef<FVector3f> Positions = Data.GetGlyph().GetStaticMeshAttributes().GetVertexPositions();

	const FVector3f& FirstPosition = Positions[FVertexID(FirstVertex)];
	const FVector2f PositionFlat = { FirstPosition.Y, FirstPosition.Z };

	Box.Min = PositionFlat;
	Box.Max = PositionFlat;


	for (int32 VertexIndex = FirstVertex + 1; VertexIndex < LastVertex; VertexIndex++)
	{
		const FVector3f& Position = Positions[FVertexID(VertexIndex)];

		Box.Min.X = FMath::Min(Box.Min.X, Position.Y);
		Box.Min.Y = FMath::Min(Box.Min.Y, Position.Z);
		Box.Max.X = FMath::Max(Box.Max.X, Position.Y);
		Box.Max.Y = FMath::Max(Box.Max.Y, Position.Z);
	}

	FStaticMeshAttributes& StaticMeshAttributes = Data.GetGlyph().GetStaticMeshAttributes();
	TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();

	auto SetTextureCoordinates = [Groups, VertexPositions, VertexInstanceUVs, &Box](const EText3DGroupType Type)
	{
		const int32 TypeFirstVertex = Groups[static_cast<int32>(Type)].FirstVertex;
		const int32 TypeLastVertex = Groups[static_cast<int32>(Type) + 1].FirstVertex;

		for (int32 Index = TypeFirstVertex; Index < TypeLastVertex; Index++)
		{
			const FVector Position = (FVector)VertexPositions[FVertexID(Index)];
			const FVector2f TextureCoordinate = (FVector2f(Position.Y, Position.Z) - Box.Min) / Box.Max;
			VertexInstanceUVs[FVertexInstanceID(Index)] = { TextureCoordinate.X, 1.f - TextureCoordinate.Y };
		}
	};

	SetTextureCoordinates(EText3DGroupType::Front);

	if (!FMath::IsNearlyZero(Bevel))
	{
		SetTextureCoordinates(EText3DGroupType::Bevel);
	}
}

void FText3DGlyphMeshBuilder::MirrorGroups(const float Extrude)
{
	// No bevels without extrude
	if (!FMath::IsNearlyZero(Extrude))
	{
		MirrorGroup(EText3DGroupType::Front, EText3DGroupType::Back, Extrude);
		MirrorGroup(EText3DGroupType::Bevel, EText3DGroupType::Bevel, Extrude);
	}
}

void FText3DGlyphMeshBuilder::MovePivot(const FVector& InNewPivot)
{
	const TVertexAttributesRef<FVector3f> VertexPositions = Data.GetGlyph().GetStaticMeshAttributes().GetVertexPositions();

	// Remove empty space around glyph first
	float MinX = TNumericLimits<float>::Max();
	float MaxX = TNumericLimits<float>::Lowest();
	float MinY = TNumericLimits<float>::Max();
	float MaxY = TNumericLimits<float>::Lowest();
	for (const FVertexID VertexID : Data.GetGlyph().GetMeshDescription().Vertices().GetElementIDs())
	{
		const FVector3f Position = VertexPositions[VertexID];
		MinX = FMath::Min(MinX, Position.X);
		MaxX = FMath::Max(MaxX, Position.X);
		MinY = FMath::Min(MinY, Position.Y);
		MaxY = FMath::Max(MaxY, Position.Y);
	}

	const FVector MeshCenter = FVector(MinX+MaxX, MinY+MaxY, 0) * 0.5;

	// Keep track of the mesh offset that we need to add back.
	MeshOffset  = +2.0 * InNewPivot * FVector(MinX, MinY, 0);
	PivotOffset = -2.0 * InNewPivot * MeshCenter;

	// Apply pivot offset with adjusted values
	const FVector3f PivotOffsetF = FVector3f(PivotOffset);
	for (const FVertexID VertexID : Data.GetGlyph().GetMeshDescription().Vertices().GetElementIDs())
	{
		VertexPositions[VertexID] += PivotOffsetF;
	}
}

void FText3DGlyphMeshBuilder::BuildMesh(FText3DCachedMesh& InMesh, EText3DMeshType InMeshType, UMaterialInterface* InDefaultMaterial)
{
	Data.GetGlyph().Build(InMesh, InMeshType, InDefaultMaterial);
	InMesh.MeshBounds = GetMeshBounds();
	InMesh.MeshOffset = GetMeshOffset();
}

FBox FText3DGlyphMeshBuilder::GetMeshBounds() const
{
	return Data.GetGlyph().GetMeshDescription().ComputeBoundingBox();
}

FVector FText3DGlyphMeshBuilder::GetMeshOffset() const
{
	return MeshOffset;
}

void FText3DGlyphMeshBuilder::CreateFrontMesh(const FText3DGlyphContourNode& Root, const bool bOutline, const float OutlineExpand, const EText3DOutlineType InOutlineType)
{
	int32 VertexCount = 0;
	AddToVertexCount(Root, VertexCount);

	Data.SetCurrentGroup(EText3DGroupType::Front, bOutline ? OutlineExpand : 0.f);
	Data.SetTarget(0.f, 0.f);
	Contours.Reset();

	int32 VertexIndex = Data.AddVertices(VertexCount);
	TriangulateAndConvert(Root, VertexIndex, bOutline, InOutlineType);

	Contours.Initialize(Data);

	if (bOutline)
	{
		CreateOutlineMesh(OutlineExpand);
	}
}

void FText3DGlyphMeshBuilder::CreateBevelMesh(const float Bevel, const EText3DBevelType Type, const int32 BevelSegments)
{
	Data.SetCurrentGroup(EText3DGroupType::Bevel, Bevel);

	if (FMath::IsNearlyZero(Bevel))
	{
		return;
	}

	switch (Type)
	{
	case EText3DBevelType::Linear:
	{
		BevelLinearWithSegments(Bevel, Bevel, BevelSegments, FVector2D(1.f, -1.f).GetSafeNormal());
		break;
	}
	case EText3DBevelType::Convex:
	{
		BevelCurve(HALF_PI, BevelSegments, [Bevel](const float CosCurr, const float SinCurr, const float CosNext, const float SinNext)
		{
				return FVector2D(CosCurr - CosNext, SinNext - SinCurr) * Bevel;
			});
		break;
	}
	case EText3DBevelType::Concave:
		{
		BevelCurve(HALF_PI, BevelSegments, [Bevel](const float CosCurr, const float SinCurr, const float CosNext, const float SinNext)
			{
				return FVector2D(SinNext - SinCurr, CosCurr - CosNext) * Bevel;
			});
		break;
			}
	case EText3DBevelType::HalfCircle:
	{
		BevelCurve(PI, BevelSegments, [Bevel](const float CosCurr, const float SinCurr, const float CosNext, const float SinNext)
		{
			return FVector2D(SinCurr - SinNext, CosCurr - CosNext) * Bevel;
		});
		break;
		}
	case EText3DBevelType::OneStep:
	{
		BevelWithSteps(Bevel, 1, BevelSegments);
		break;
	}
	case EText3DBevelType::TwoSteps:
	{
		BevelWithSteps(Bevel, 2, BevelSegments);
		break;
	}
	case EText3DBevelType::Engraved:
	{
		BevelLinearWithSegments(-Bevel, 0.f, BevelSegments, FVector2D(-1.f, 0.f));
		BevelLinearWithSegments(0.f, Bevel, BevelSegments, FVector2D(0.f, -1.f));
		BevelLinearWithSegments(Bevel, 0.f, BevelSegments, FVector2D(1.f, 0.f));
		break;
	}
	default:
		break;
	}
}

void FText3DGlyphMeshBuilder::CreateExtrudeMesh(float Extrude, float Bevel, const EText3DBevelType Type, bool bFlipNormals)
{
	if (FMath::IsNearlyZero(Extrude))
	{
		return;
	}

	Bevel = FMath::Max(UE_SMALL_NUMBER, Bevel);

	if (Type != EText3DBevelType::HalfCircle)
	{
		Bevel = FMath::Clamp(Bevel, 0.0f, Extrude / 2.f);
	}

	if (Type != EText3DBevelType::HalfCircle && Type != EText3DBevelType::Engraved)
	{
		Extrude -= Bevel * 2.0f;
	}

	Data.SetCurrentGroup(EText3DGroupType::Extrude, 0.f);

	const FVector2D Normal(1.f, 0.f);
	Data.PrepareSegment(Extrude, 0.f, Normal, Normal);

	Contours.Reset();


	TArray<float> TextureCoordinateVs;

	for (FText3DGlyphContour& Contour : Contours)
	{
		// Compute TexCoord.V-s for each point
		TextureCoordinateVs.Reset(Contour.Num() - 1);
		const FText3DGlyphPartPtr First = Contour[0];
		TextureCoordinateVs.Add(First->Length());

		int32 Index = 1;
		for (FText3DGlyphPartConstPtr Edge = First->Next; Edge != First->Prev; Edge = Edge->Next)
		{
			TextureCoordinateVs.Add(TextureCoordinateVs[Index - 1] + Edge->Length());
			Index++;
		}


		const float ContourLength = TextureCoordinateVs.Last() + Contour.Last()->Length();

		if (FMath::IsNearlyZero(ContourLength))
		{
			continue;
		}


		for (float& PointY : TextureCoordinateVs)
		{
			PointY /= ContourLength;
		}

		// Duplicate contour
		Data.SetTarget(0.f, 0.f);
		const bool bFirstSmooth = First->bSmooth;
		// It's set to sharp because we need 2 vertices with TexCoord.Y values 0 and 1 (for smooth points only one vertex is added)
		First->bSmooth = false;

		// First point in contour is processed separately
		{
			EmptyPaths(First);
			ExpandPointWithoutAddingVertices(First);

			const FVector2D TexCoordPrev(0.f, 0.f);
			const FVector2D TexCoordCurr(0.f, 1.f);

			if (bFirstSmooth)
			{
				AddVertexSmooth(First, TexCoordPrev);
				AddVertexSmooth(First, TexCoordCurr);
			}
			else
			{
				AddVertexSharp(First, First->Prev, TexCoordPrev);
				AddVertexSharp(First, First, TexCoordCurr);
			}
		}

		Index = 1;
		for (FText3DGlyphPartPtr Point = First->Next; Point != First; Point = Point->Next)
		{
			EmptyPaths(Point);
			ExpandPoint(Point, {0.f, 1.f - TextureCoordinateVs[Index++ - 1]});
		}

		// Add extruded vertices
		Data.SetTarget(Data.GetPlannedExtrude(), Data.GetPlannedExpand());

		// Similarly to duplicating vertices, first point is processed separately
		{
			ExpandPointWithoutAddingVertices(First);

			const FVector2D TexCoordPrev(1.f, 0.f);
			const FVector2D TexCoordCurr(1.f, 1.f);

			if (bFirstSmooth)
			{
				AddVertexSmooth(First, TexCoordPrev);
				AddVertexSmooth(First, TexCoordCurr);
			}
			else
			{
				AddVertexSharp(First, First->Prev, TexCoordPrev);
				AddVertexSharp(First, First, TexCoordCurr);
			}
		}

		Index = 1;
		for (FText3DGlyphPartPtr Point = First->Next; Point != First; Point = Point->Next)
		{
			ExpandPoint(Point, {1.f, 1.f - TextureCoordinateVs[Index++ - 1]});
		}

		for (const FText3DGlyphPartPtr& Edge : Contour)
		{
			Data.FillEdge(Edge, false, bFlipNormals);
		}
	}
}

void FText3DGlyphMeshBuilder::MirrorGroup(const EText3DGroupType TypeIn, const EText3DGroupType TypeOut, const float Extrude)
{
	TText3DGroupList& Groups = Data.GetGlyph().GetGroups();

	const FText3DPolygonGroup GroupIn = Groups[static_cast<int32>(TypeIn)];
	const FText3DPolygonGroup GroupNext = Groups[static_cast<int32>(TypeIn) + 1];

	const int32 VerticesInNum = GroupNext.FirstVertex - GroupIn.FirstVertex;
	const int32 TrianglesInNum = GroupNext.FirstTriangle - GroupIn.FirstTriangle;

	FMeshDescription& MeshDescription = Data.GetGlyph().GetMeshDescription();
	const int32 TotalVerticesNum = MeshDescription.Vertices().Num();

	Data.SetCurrentGroup(TypeOut, 0.f);
	Data.AddVertices(VerticesInNum);

	FStaticMeshAttributes& StaticMeshAttributes = Data.GetGlyph().GetStaticMeshAttributes();
	TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexNormals = StaticMeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexTangents = StaticMeshAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<FVector2f> VertexUVs = StaticMeshAttributes.GetVertexInstanceUVs();

	for (int32 VertexIndex = 0; VertexIndex < VerticesInNum; VertexIndex++)
	{
		const FVertexID VertexID(GroupIn.FirstVertex + VertexIndex);
		const FVertexInstanceID InstanceID(static_cast<uint32>(VertexID.GetValue()));

		const FVector Position = (FVector)VertexPositions[VertexID];
		const FVector Normal = (FVector)VertexNormals[InstanceID];
		const FVector Tangent = (FVector)VertexTangents[InstanceID];

		Data.AddVertex({ Extrude - Position.X, Position.Y, Position.Z }, { -Tangent.X, Tangent.Y, Tangent.Z }, { -Normal.X, Normal.Y, Normal.Z }, FVector2D(VertexUVs[InstanceID]));
	}

	Data.AddTriangles(TrianglesInNum);

	for (int32 TriangleIndex = 0; TriangleIndex < TrianglesInNum; TriangleIndex++)
	{
		const FTriangleID TriangleID = FTriangleID(GroupIn.FirstTriangle + TriangleIndex);
		TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription.GetTriangleVertexInstances(TriangleID);

		uint32 Instance0 = static_cast<uint32>(TotalVerticesNum + VertexInstanceIDs[0].GetValue() - GroupIn.FirstVertex);
		uint32 Instance2 = static_cast<uint32>(TotalVerticesNum + VertexInstanceIDs[2].GetValue() - GroupIn.FirstVertex);
		uint32 Instance1 = static_cast<uint32>(TotalVerticesNum + VertexInstanceIDs[1].GetValue() - GroupIn.FirstVertex);
		Data.AddTriangle(Instance0, Instance2, Instance1);
	}
}

void FText3DGlyphMeshBuilder::AddToVertexCount(const FText3DGlyphContourNode& Node, int32& OutVertexCount)
{
	for (const FText3DGlyphContourNode& Child : Node.Children)
	{
		OutVertexCount += Child.Contour->VertexCount();
		AddToVertexCount(Child, OutVertexCount);
	}
}

void FText3DGlyphMeshBuilder::TriangulateAndConvert(const FText3DGlyphContourNode& Node, int32& OutVertexIndex, const bool bOutline, const EText3DOutlineType InOutlineType)
{
	// If this is solid region
	if (!Node.bClockwise)
	{
		int32 VertexCount = 0;
		UE::Geometry::FConstrainedDelaunay2f Triangulation;
		Triangulation.FillRule = UE::Geometry::FConstrainedDelaunay2f::EFillRule::Positive;

		auto ProcessContour = [&Contours=Contours, &Data=Data, &VertexCount, &Triangulation, bOutline, InOutlineType](const FText3DGlyphContourNode& NodeIn)
		{
			// Create contour in old format
			FText3DGlyphContour& Contour = Contours.Add();
			const FPolygon2f& Polygon = *NodeIn.Contour;

			for (const FVector2f& Vertex : Polygon.GetVertices())
			{
				// Add point to contour in old format
				const FText3DGlyphPartPtr Point = MakeShared<FText3DGlyphPart>();
				Contour.Add(Point);
				Point->Position = FVector2D(Vertex);

				// Add point to mesh
				const int32 VertexID = Data.AddVertex(Point->Position, { 1.f, 0.f }, { -1.f, 0.f, 0.f });

				Point->PathPrev.Add(VertexID);
				Point->PathNext.Add(VertexID);
			}

			VertexCount += Polygon.VertexCount();

			// Add contour to triangulation
			if (!bOutline || InOutlineType != EText3DOutlineType::Stroke)
			{
				Triangulation.Add(Polygon, NodeIn.bClockwise);
			}
		};

		// Outer
		ProcessContour(Node);

		// Holes
		for (const FText3DGlyphContourNode& Child : Node.Children)
		{
			ProcessContour(Child);
		}

		if (!bOutline || InOutlineType != EText3DOutlineType::Stroke)
		{
			Triangulation.Triangulate();
			const TArray<FIndex3i>& Triangles = Triangulation.Triangles;
			Data.AddTriangles(Triangles.Num());

			for (const FIndex3i& Triangle : Triangles)
			{
				Data.AddTriangle(OutVertexIndex + Triangle.A, OutVertexIndex + Triangle.C, OutVertexIndex + Triangle.B);
			}
		}

		OutVertexIndex += VertexCount;
	}

	// Continue with children
	for (const FText3DGlyphContourNode& Child : Node.Children)
	{
		TriangulateAndConvert(Child, OutVertexIndex, bOutline, InOutlineType);
	}
}

void FText3DGlyphMeshBuilder::CreateOutlineMesh(float InOutlineExpand)
{
	Data.SetCurrentGroup(EText3DGroupType::Bevel, InOutlineExpand);

	FText3DGlyphContourList InitialContours = Contours;
	for (FText3DGlyphContour& Contour : InitialContours)
	{
		Algo::Reverse(Contour);

		for (FText3DGlyphPartPtr& Point : Contour)
		{
			Swap(Point->Prev, Point->Next);
			Point->Normal *= -1.f;
		}

		const FText3DGlyphPartPtr First = Contour[0];
		const FText3DGlyphPartPtr Last = Contour.Last();

		const FVector2D FirstTangentX = First->TangentX;

		for (FText3DGlyphPartPtr Edge = First; Edge != Last; Edge = Edge->Next)
		{
			Edge->TangentX = -Edge->Next->TangentX;
		}

		Last->TangentX = -FirstTangentX;
	}

	const FVector2D Normal = {0.f, -1.f};
	BevelLinear(0.f, InOutlineExpand, Normal, Normal, false);

	Contours.Reset();

	TDoubleLinkedList<FText3DGlyphContour>::TDoubleLinkedListNode* Node = InitialContours.GetHead();
	while (Node)
	{
		Contours.AddTail(Node);
		InitialContours.RemoveNode(Node, false);

		Node = InitialContours.GetHead();
	}
}

void FText3DGlyphMeshBuilder::BevelLinearWithSegments(const float Extrude, const float Expand, const int32 BevelSegments, const FVector2D Normal)
{
	for (int32 Index = 0; Index < BevelSegments; Index++)
	{
		BevelLinear(Extrude / BevelSegments, Expand / BevelSegments, Normal, Normal, false);
	}
}

void FText3DGlyphMeshBuilder::BevelCurve(const float Angle, const int32 BevelSegments, TFunction<FVector2D(const float CurrentCos, const float CurrentSin, const float NextCos, const float Next)> ComputeOffset)
{
	float CosCurr = 0.0f;
	float SinCurr = 0.0f;

	float CosNext = 0.0f;
	float SinNext = 0.0f;

	FVector2D OffsetNext;
	bool bSmoothNext = false;

	FVector2D NormalNext;
	FVector2D NormalEnd;

	auto UpdateAngle = [Angle, &CosNext, &SinNext, BevelSegments](const int32 Index)
	{
		const float Step = Angle / BevelSegments;
		FMath::SinCos(&SinNext, &CosNext, Index * Step);
	};

	auto MakeStep = [UpdateAngle, &OffsetNext, ComputeOffset, &CosCurr, &SinCurr, &CosNext, &SinNext, &NormalNext](int32 Index)
	{
		UpdateAngle(Index);
		OffsetNext = ComputeOffset(CosCurr, SinCurr, CosNext, SinNext);
		NormalNext = FVector2D(OffsetNext.X, -OffsetNext.Y).GetSafeNormal();
	};


	UpdateAngle(0);

	CosCurr = CosNext;
	SinCurr = SinNext;

	MakeStep(1);
	for (int32 Index = 0; Index < BevelSegments; Index++)
	{
		CosCurr = CosNext;
		SinCurr = SinNext;

		const FVector2D OffsetCurr = OffsetNext;

		const FVector2D NormalCurr = NormalNext;
		FVector2D NormalStart;

		const bool bFirst = (Index == 0);
		const bool bLast = (Index == BevelSegments - 1);

		const bool bSmooth = bSmoothNext;

		if (!bLast)
		{
			MakeStep(Index + 2);
			bSmoothNext = FVector2D::DotProduct(NormalCurr, NormalNext) >= -FText3DGlyphPart::CosMaxAngleSides;
		}

		NormalStart = bFirst ? NormalCurr : (bSmooth ? NormalEnd : NormalCurr);
		NormalEnd = bLast ? NormalCurr : (bSmoothNext ? (NormalCurr + NormalNext).GetSafeNormal() : NormalCurr);

		BevelLinear(OffsetCurr.X, OffsetCurr.Y, NormalStart, NormalEnd, bSmooth);
	}
}

void FText3DGlyphMeshBuilder::BevelWithSteps(const float Bevel, const int32 Steps, const int32 BevelSegments)
{
	const float BevelPerStep = Bevel / Steps;

	for (int32 Step = 0; Step < Steps; Step++)
	{
		BevelLinearWithSegments(BevelPerStep, 0.f, BevelSegments, FVector2D(1.f, 0.f));
		BevelLinearWithSegments(0.f, BevelPerStep, BevelSegments, FVector2D(0.f, -1.f));
	}
}

void FText3DGlyphMeshBuilder::BevelLinear(const float Extrude, const float Expand, FVector2D NormalStart, FVector2D NormalEnd, const bool bSmooth)
{
	Data.PrepareSegment(Extrude, Expand, NormalStart, NormalEnd);
	Contours.Reset();

	if (!bSmooth)
	{
		DuplicateContourVertices();
	}

	BevelPartsWithoutIntersectingNormals();

	Data.IncreaseDoneExtrude();
}

void FText3DGlyphMeshBuilder::DuplicateContourVertices()
{
	Data.SetTarget(0.f, 0.f);

	for (FText3DGlyphContour& Contour : Contours)
	{
		for (const FText3DGlyphPartPtr& Point : Contour)
		{
			EmptyPaths(Point);
			// Duplicate points of contour (expansion with value 0)
			ExpandPoint(Point);
		}
	}
}

void FText3DGlyphMeshBuilder::BevelPartsWithoutIntersectingNormals()
{
	Data.SetTarget(Data.GetPlannedExtrude(), Data.GetPlannedExpand());
	const float MaxExpand = Data.GetPlannedExpand();

	const bool bFlipNormals = FMath::Sign(Data.GetPlannedExpand()) < 0;
	for (FText3DGlyphContour& Contour : Contours)
	{
		for (const FText3DGlyphPartPtr& Point : Contour)
		{
			if (!FMath::IsNearlyEqual(Point->DoneExpand, MaxExpand) || FMath::IsNearlyZero(MaxExpand))
			{
				ExpandPoint(Point);
			}

			const float Delta = MaxExpand - Point->DoneExpand;

			Point->AvailableExpandNear -= Delta;
			Point->DecreaseExpandsFar(Delta);
		}

		for (const FText3DGlyphPartPtr& Edge : Contour)
		{
			Data.FillEdge(Edge, false, bFlipNormals);
		}
	}
}

void FText3DGlyphMeshBuilder::EmptyPaths(const FText3DGlyphPartPtr& Point) const
{
	Point->PathPrev.Empty();
	Point->PathNext.Empty();
}

void FText3DGlyphMeshBuilder::ExpandPoint(const FText3DGlyphPartPtr& Point, const FVector2D TextureCoordinates)
{
	ExpandPointWithoutAddingVertices(Point);

	if (Point->bSmooth)
	{
		AddVertexSmooth(Point, TextureCoordinates);
	}
	else
	{
		AddVertexSharp(Point, Point->Prev, TextureCoordinates);
		AddVertexSharp(Point, Point, TextureCoordinates);
	}
}

void FText3DGlyphMeshBuilder::ExpandPointWithoutAddingVertices(const FText3DGlyphPartPtr& Point)
{
	Point->Position = Data.Expanded(Point);
	const int32 FirstAdded = Data.AddVertices(Point->bSmooth ? 1 : 2);

	Point->PathPrev.Add(FirstAdded);
	Point->PathNext.Add(Point->bSmooth ? FirstAdded : FirstAdded + 1);
}

void FText3DGlyphMeshBuilder::AddVertexSmooth(const FText3DGlyphPartConstPtr& Point, const FVector2D TextureCoordinates)
{
	const FText3DGlyphPartConstPtr Curr = Point;
	const FText3DGlyphPartConstPtr Prev = Point->Prev;

	Data.AddVertex(Point, (Prev->TangentX + Curr->TangentX).GetSafeNormal(), (Data.ComputeTangentZ(Prev, Point->DoneExpand) + Data.ComputeTangentZ(Curr, Point->DoneExpand)).GetSafeNormal(), TextureCoordinates);
}

void FText3DGlyphMeshBuilder::AddVertexSharp(const FText3DGlyphPartConstPtr& Point, const FText3DGlyphPartConstPtr& Edge, const FVector2D TextureCoordinates)
{
	Data.AddVertex(Point, Edge->TangentX, Data.ComputeTangentZ(Edge, Point->DoneExpand).GetSafeNormal(), TextureCoordinates);
}
