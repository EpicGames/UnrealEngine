// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBuilders/Text3DGlyphContourNode.h"
#include "GeometryBuilders/Text3DGlyphPart.h"
#include "Text3DGlyphContourList.h"
#include "Text3DGlyphData.h"
#include "Text3DTypes.h"

class FText3DGlyph;
class UMaterial;
class UMaterialInterface;
class UStaticMesh;
struct FText3DCachedMesh;

/** Creates the glyph mesh from glyph data */
class FText3DGlyphMeshBuilder final
{
public:
	/**
	 * CreateMeshes
	 * @param InRoot - Tree of contours.	 
	 * @param InExtrude - Orthogonal (to front cap) offset value.
	 * @param InBevel - Bevel value (bevel happens before extrude).
	 * @param InBevelType - Defines shape of beveled part.
	 * @param InBevelSegments - Segments count.
	 * @param bInOutline - Front face has outline and is not filled.
	 * @param InOutlineExpand - Offsets the outline by the specified amount.
	 * @param InOutlineType - Defines the outline behavior.
	 */
	void CreateMeshes(const FText3DGlyphContourNode& InRoot, const float InExtrude, const float InBevel, const EText3DBevelType InBevelType, const int32 InBevelSegments, const bool bInOutline, const float InOutlineExpand, const EText3DOutlineType InOutlineType);
	void SetFrontAndBevelTextureCoordinates(const float Bevel);
	void MirrorGroups(const float Extrude);

	/** Move the mesh pivot based on pivot ratio 0-1 */
	void MovePivot(const FVector& InNewPivot);

	/** Creates the actual mesh */
	void BuildMesh(FText3DCachedMesh& InMesh, EText3DMeshType InMeshType, UMaterialInterface* InDefaultMaterial);

	/** Get actual glyph bounds once built */
	FBox GetMeshBounds() const;

	/** Get actual glyph offset */
	FVector GetMeshOffset() const;

private:
	FVector MeshOffset = FVector::ZeroVector;
	FVector PivotOffset = FVector::ZeroVector;
	FText3DGlyphData Data;
	FText3DGlyphContourList Contours;

	/**
	 * Create 'Front' part of glyph.
	 * @param Root - Tree root.
	 * @param bOutline - Front face has outline and is not filled.
	 * @param OutlineExpand - Offsets the outline by the specified amount.
	 * @param InOutlineType - Defines the outline behavior.
	 */
	void CreateFrontMesh(const FText3DGlyphContourNode& Root, const bool bOutline, const float OutlineExpand, const EText3DOutlineType InOutlineType);

	/**
	 * Create 'Bevel' part of glyph (actually half of it, will be mirrored later).
	 * @param Bevel - Bevel value (bevel happens before extrude).
	 * @param Type - Defines shape of beveled part.
	 * @param BevelSegments - Segments count.
	 */
	void CreateBevelMesh(const float Bevel, const EText3DBevelType Type, const int32 BevelSegments);

	/**
	 * Create 'Extrude' part of glyph.
	 * @param Extrude - Extrude value.
	 * @param Bevel - Bevel value (bevel happens before extrude).
	 * @param Type - Defines shape of beveled part.
	 * @param bFlipNormals - Reverse the geometry facing direction.
	 */
	void CreateExtrudeMesh(float Extrude, float Bevel, const EText3DBevelType Type, bool bFlipNormals = false);

	/**
	 * Create 'Outline' part of glyph. Not compatible with Bevel
	 * @param InOutlineExpand - outline expansion value.
	 */
	void CreateOutlineMesh(float InOutlineExpand);

	/**
	 * Copies and reflect source group into a destination group based on extrusion value
	 */
	void MirrorGroup(const EText3DGroupType TypeIn, const EText3DGroupType TypeOut, const float Extrude);

	/**
	 * Recursively compute vertex count in tree.
	 * @param Node - Tree node.
	 * @param OutVertexCount - Pointer to counter.
	 */
	void AddToVertexCount(const FText3DGlyphContourNode& Node, int32& OutVertexCount);

	/**
	 * Triangulate solid region with holes and convert contours to old format.
	 * @param Node - Tree node.
	 * @param OutVertexIndex - Pointer to last used vertex index.
	 * @param bOutline - Front face has outline and is not filled.
	 */
	void TriangulateAndConvert(const FText3DGlyphContourNode& Node, int32& OutVertexIndex, const bool bOutline, const EText3DOutlineType InOutlineType);

	void BevelLinearWithSegments(const float Extrude, const float Expand, const int32 BevelSegments, const FVector2D Normal);
	void BevelCurve(const float Angle, const int32 BevelSegments, TFunction<FVector2D(const float CurrentCos, const float CurrentSin, const float NextCos, const float Next)> ComputeOffset);
	void BevelWithSteps(const float Bevel, const int32 Steps, const int32 BevelSegments);

	/**
	 * Bevel one segment.
	 * @param Extrude - Documented in ContourList.h.
	 * @param Expand - Documented in ContourList.h.
	 * @param NormalStart - Normal at start of segment (minimum DoneExpand).
	 * @param NormalEnd - Normal at end of segment (maximum DoneExpand).
	 * @param bSmooth - Is angle between start of this segment and end of previous segment smooth?
	 */
	void BevelLinear(const float Extrude, const float Expand, FVector2D NormalStart, FVector2D NormalEnd, const bool bSmooth);

	/**
	 * Duplicate contour vertices (used to make sharp angle between bevel steps)
	 */
	void DuplicateContourVertices();

	/**
	 * Continue with trivial bevel till FData::Expand.
	 */
	void BevelPartsWithoutIntersectingNormals();

	/**
	 * Clear PathPrev and PathNext.
	 * @param Point - Point which paths should be cleared.
	 */
	void EmptyPaths(const FText3DGlyphPartPtr& Point) const;

	/**
	 * Same as previous function but does not cover the intersection-case.
	 * @param Point - Point that should be expanded.
	 * @param TextureCoordinates - Texcture coordinates of added vertices.
	 */
	void ExpandPoint(const FText3DGlyphPartPtr& Point, const FVector2D TextureCoordinates = FVector2D(0.f, 0.f));

	/**
	 * Common code for expanding, vertices are added uninitialized.
	 * @param Point - Expanded point.
	 */
	void ExpandPointWithoutAddingVertices(const FText3DGlyphPartPtr& Point);

	/**
	 * Add vertex for smooth point.
	 * @param Point - Expanded point.
	 * @param TextureCoordinates - Texture coordinates of added vertex.
	 */
	void AddVertexSmooth(const FText3DGlyphPartConstPtr& Point, const FVector2D TextureCoordinates);

	/**
	 * Add vertex for sharp point.
	 * @param Point - Expanded point.
	 * @param Edge - Edge from which TangentX and TangentZ will be assigned.
	 * @param TextureCoordinates - Texture coordinates of added vertex.
	 */
	void AddVertexSharp(const FText3DGlyphPartConstPtr& Point, const FText3DGlyphPartConstPtr& Edge, const FVector2D TextureCoordinates);
};
