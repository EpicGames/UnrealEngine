// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DGlyphOutline.h"

namespace UE::Text3D
{

#if WITH_FREETYPE
	FGlyphOutline FGlyphOutline::MakeOutline(const FT_Outline& InOutline)
	{
		FGlyphOutline GlyphOutline;
		GlyphOutline.Points = TArray<FT_Vector>(InOutline.points, InOutline.n_points);
		GlyphOutline.Tags = TArray<FFTOutlineTagsType>(InOutline.tags, InOutline.n_points);
		GlyphOutline.Contours = TArray<FFTOutlineContoursType>(InOutline.contours, InOutline.n_contours);
		GlyphOutline.Flags = InOutline.flags;
		return GlyphOutline;
	}
#endif

} // UE::Text3D
