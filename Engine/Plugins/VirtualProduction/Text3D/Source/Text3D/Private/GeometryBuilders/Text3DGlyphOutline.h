// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/NotNull.h"
#include "Text3DModule.h"

namespace UE::Text3D
{

#if WITH_FREETYPE
	// @TODOFonts: The internal types of FT_Outline has changed at FreeType2-2.13.3
	// The most notable are the type for FT_Outline.Contours and FT_Outline.Tags from short* to unsigned short* and char* to unsigned char* respectively
	// We need to support 2.10.0 and 2.14.1 for now
	// All of this should be cleaned up after fully migrating to 2.14.1 across all platforms 
#if FREETYPE_MAJOR >= 2 && FREETYPE_MINOR >= 14
	using FFTOutlineTagsType = unsigned char;
	using FFTOutlineContoursType = unsigned short;
#else
	using FFTOutlineTagsType = char;
	using FFTOutlineContoursType = short;
#endif

	/**
	 * Mirrors the struct of 'FT_Outline' to be used outside of the loaded glyph.
	 * FT_Load_Glyph is not thread-safe when using the same FT_Face, so the data needs to be saved in this struct for
	 * expensive computations to be done in multiple threads.
	 */
	struct FGlyphOutline
	{
		static FGlyphOutline MakeOutline(const FT_Outline& InOutline);

		/** Outline's points */
		TArray<FT_Vector> Points;

		/** Points flags */
		TArray<FFTOutlineTagsType> Tags;

		/** Contour end points */
		TArray<FFTOutlineContoursType> Contours;

		/** Outline Masks */
		int32 Flags = 0;
	};
#endif

} // UE::Text3D
