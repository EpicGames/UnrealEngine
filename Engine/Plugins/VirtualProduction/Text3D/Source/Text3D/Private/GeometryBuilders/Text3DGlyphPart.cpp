// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryBuilders/Text3DGlyphPart.h"

constexpr float FText3DGlyphPart::CosMaxAngleSideTangent;
constexpr float FText3DGlyphPart::CosMaxAngleSides;

FText3DGlyphPart::FText3DGlyphPart()
{
	Position = FVector2D::ZeroVector;
	TangentX = FVector2D::ZeroVector;
	Normal = FVector2D::ZeroVector;
	InitialPosition = FVector2D::ZeroVector;
	Prev = nullptr;
	Next = nullptr;
	bSmooth = false;
	AvailableExpandNear = 0.0f;
	DoneExpand = 0.f;
}

FText3DGlyphPart::FText3DGlyphPart(const FText3DGlyphPartConstPtr& Other)
{
	Position = Other->Position;
	DoneExpand = Other->DoneExpand;
	TangentX = Other->TangentX;
	Normal = Other->Normal;
	bSmooth = Other->bSmooth;
	InitialPosition = Other->InitialPosition;
	PathPrev = Other->PathPrev;
	PathNext = Other->PathNext;
	AvailableExpandNear = Other->AvailableExpandNear;
}

float FText3DGlyphPart::TangentsDotProduct() const
{
	check(Prev);
	return FVector2D::DotProduct(-Prev->TangentX, TangentX);
}

float FText3DGlyphPart::Length() const
{
	check(Next);

	return (Next->Position - Position).Size();
}

void FText3DGlyphPart::ResetDoneExpand()
{
	DoneExpand = 0.f;
}

void FText3DGlyphPart::ComputeTangentX()
{
	check(Next);
	TangentX = (Next->Position - Position).GetSafeNormal();
}

bool FText3DGlyphPart::ComputeNormal()
{
	check(Prev);

	// Scale is needed to make ((p_(i+1) + k * n_(i+1)) - (p_i + k * n_i)) parallel to (p_(i+1) - p_i). Also (k) is distance between original edge and this edge after expansion with value (k).
	const float OneMinusADotC = 1.0f - TangentsDotProduct();

	if (FMath::IsNearlyZero(OneMinusADotC))
	{
		return false;
	}

	const FVector2D A = -Prev->TangentX;
	const FVector2D C = TangentX;

	Normal = A + C;
	const float NormalLength2 = Normal.SizeSquared();
	const float Scale = -FPlatformMath::Sqrt(2.0f / OneMinusADotC);

	// If previous and next edge are nearly on one line
	if (FMath::IsNearlyZero(NormalLength2, 0.0001f))
	{
		Normal = FVector2D(A.Y, -A.X) * Scale;
	}
	else
	{
		// Sign of cross product is needed to be sure that Normal is directed outside.
		Normal *= -Scale * FPlatformMath::Sign(FVector2D::CrossProduct(A, C)) / FPlatformMath::Sqrt(NormalLength2);
	}

	Normal.Normalize();

	return true;
}

void FText3DGlyphPart::ComputeSmooth()
{
	bSmooth = TangentsDotProduct() <= CosMaxAngleSides;
}

bool FText3DGlyphPart::ComputeNormalAndSmooth()
{
	if (!ComputeNormal())
	{
		return false;
	}

	ComputeSmooth();
	return true;
}

void FText3DGlyphPart::ResetInitialPosition()
{
	InitialPosition = Position;
}

void FText3DGlyphPart::ComputeInitialPosition()
{
	InitialPosition = Position - DoneExpand * Normal;
}

void FText3DGlyphPart::DecreaseExpandsFar(const float Delta)
{
	for (auto It = AvailableExpandsFar.CreateIterator(); It; ++It)
	{
		It->Value -= Delta;

		if (It->Value < 0.f)
		{
			It.RemoveCurrent();
		}
	}
}

FVector2D FText3DGlyphPart::Expanded(const float Value) const
{
	return Position + Normal * Value;
}
