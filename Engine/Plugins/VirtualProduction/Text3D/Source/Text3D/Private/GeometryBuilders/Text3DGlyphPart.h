// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

struct FText3DGlyphPart;

using FText3DGlyphPartPtr = TSharedPtr<FText3DGlyphPart>;
using FText3DGlyphPartConstPtr = TSharedPtr<const FText3DGlyphPart>;

/** Part represents point and it's next edge (counterclockwise one). */
struct FText3DGlyphPart final
{
	static constexpr float CosMaxAngleSideTangent = 0.995f;
	static constexpr float CosMaxAngleSides = -0.9f;
	
	class FAvailableExpandsFar final : public TMap<FText3DGlyphPartPtr, float>
	{
	public:
		void Add(const FText3DGlyphPartConstPtr Edge, const float Value)
		{
			TMap<FText3DGlyphPartPtr, float>::Add(ConstCastSharedPtr<FText3DGlyphPart>(Edge), Value);
		}

		void Remove(const FText3DGlyphPartConstPtr Edge)
		{
			TMap<FText3DGlyphPartPtr, float>::Remove(ConstCastSharedPtr<FText3DGlyphPart>(Edge));
		}
	};

	FText3DGlyphPart();
	FText3DGlyphPart(const FText3DGlyphPartConstPtr& Other);

	/** Previous part. */
	FText3DGlyphPartPtr Prev;

	/** Next part. */
	FText3DGlyphPartPtr Next;

	/** Position, is equal to position of last vertex in paths (in coordinate system of glyph). */
	FVector2D Position;

	/** Offset in surface of front cap that this point already made. */
	float DoneExpand;

	FVector2D TangentX;

	/** Point normal, a bisector of angle. */
	FVector2D Normal;

	/** If true, previous and next edges are in one smoothing group. */
	bool bSmooth;

	FVector2D InitialPosition;

	// Paths along which triangulation is made. Values that are stored are indices of vertices. If bSmooth == true, both paths store one index (for same DoneExpand value). If not, indices are different.
	/** Path used for triangulation of previous edge. */
	TArray<int32> PathPrev;

	/** Path used for triangulation of next edge. */
	TArray<int32> PathNext;

	/** Offset needed for an IntersectionNear to happen. */
	float AvailableExpandNear;

	/** List of pairs (edge, offset) for IntersectionFar. */
	FAvailableExpandsFar AvailableExpandsFar;

	float TangentsDotProduct() const;
	float Length() const;

	void ResetDoneExpand();
	void ComputeTangentX();
	bool ComputeNormal();
	void ComputeSmooth();

	bool ComputeNormalAndSmooth();

	void ResetInitialPosition();
	void ComputeInitialPosition();

	void DecreaseExpandsFar(const float Delta);

	/**
	 * Compute position to which point will be expanded.
	 * @param Value - Offset in surface of front cap.
	 * @return Computed position.
	 */
	FVector2D Expanded(const float Value) const;
};
