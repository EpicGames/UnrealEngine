// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DPointHashGrid2d.h"

namespace UE::Text3D
{

FPointHashGrid2d::FPointHashGrid2d(double InCellSize, int32 InInvalidValue)
	: Indexer(InCellSize), InvalidValue(InInvalidValue)
{
}

int32 FPointHashGrid2d::GetInvalidValue() const
{
	return InvalidValue;
}

void FPointHashGrid2d::InsertPoint(int32 InValue, const FVector2D& InPosition)
{
	UE::Geometry::FVector2i Index = Indexer.ToGrid(InPosition);
	Hash.Add(Index, InValue);
}

TPair<int32, double> FPointHashGrid2d::FindNearestInRadius(const FVector2D& InQueryPoint, double InRadius, TFunctionRef<double(int32)> InDistanceSqFunc, TFunctionRef<bool(int32)> InIgnoreFunc) const
{
	if (Hash.IsEmpty())
	{
		return TPair<int32, double>(GetInvalidValue(), TNumericLimits<double>::Max());
	}

	const UE::Geometry::FVector2i MinIndex = Indexer.ToGrid(InQueryPoint - InRadius * FVector2D::One());
	const UE::Geometry::FVector2i MaxIndex = Indexer.ToGrid(InQueryPoint + InRadius * FVector2D::One());
	const double RadiusSquared = InRadius * InRadius;

	int32 Nearest = GetInvalidValue();
	double MinDistanceSq = TNumericLimits<double>::Max();

	TArray<int32> Values;

	for (int32 IndexY = MinIndex.Y; IndexY <= MaxIndex.Y; ++IndexY) 
	{
		for (int32 IndexX = MinIndex.X; IndexX <= MaxIndex.X; ++IndexX) 
		{
			UE::Geometry::FVector2i Index(IndexX, IndexY);
			Values.Reset();

			Hash.MultiFind(Index, Values);
			for (int32 Value : Values) 
			{
				if (InIgnoreFunc(Value))
				{
					continue;
				}
				const double DistanceSquared = InDistanceSqFunc(Value);
				if (DistanceSquared < RadiusSquared && DistanceSquared < MinDistanceSq)
				{
					Nearest = Value;
					MinDistanceSq = DistanceSquared;
				}
			}
		}
	}
	return TPair<int32, double>(Nearest, MinDistanceSq);
}

} // UE::Text3D
