// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IntVectorTypes.h"
#include "Containers/Map.h"
#include "Util/GridIndexing2.h"

namespace UE::Text3D
{

/**
 * This is a simplified re-implementation of TPointHashGrid2 
 * but without the FCriticalSection to allow for copying and because it was unnecessary in our context.
 */
class FPointHashGrid2d
{
public:
	/**
	 * Construct 2D hash grid
	 * @param InCellSize size of grid cells
	 * @param InInvalidValue this value will be returned by queries if no valid result is found (e.g. bounded-distance query)
	 */
	FPointHashGrid2d(double InCellSize, int32 InInvalidValue);

	/** Invalid grid value */
	int32 GetInvalidValue() const;

	/**
	 * Insert at given position.
	 * @param InValue the point/value to insert
	 * @param InPosition the position associated with this value
	 */
	void InsertPoint(int32 InValue, const FVector2D& InPosition);

	/**
	 * Find nearest point in grid, within a given sphere
	 * @param InQueryPoint the center of the query sphere
	 * @param InRadius the radius of the query sphere
	 * @param InDistanceSqFunc Function you provide which measures the distance between QueryPoint and a Value
	 * @param InIgnoreFunc optional Function you may provide which will result in a Value being ignored if IgnoreFunc(Value) returns true
	 * @return the found pair (Value,DistanceFunc(Value)), or (InvalidValue,MaxDouble) if not found
	 */
	TPair<int32, double> FindNearestInRadius(const FVector2D& InQueryPoint, double InRadius
		, TFunctionRef<double(int32)> InDistanceSqFunc
		, TFunctionRef<bool(int32)> InIgnoreFunc = [](const int32 data) { return false; }) const;

private:
	TMultiMap<UE::Geometry::FVector2i, int32> Hash;
	UE::Geometry::FScaleGridIndexer2d Indexer;
	int32 InvalidValue;
};

} // UE::Text3D
