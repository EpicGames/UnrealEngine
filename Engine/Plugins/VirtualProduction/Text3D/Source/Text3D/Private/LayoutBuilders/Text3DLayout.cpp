// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutBuilders/Text3DLayout.h"

#include "Brushes/SlateNoResource.h"
#include "Framework/Text/SlateTextRun.h"

FText3DLayout::FText3DLayout(const FTextBlockStyle& InStyle)
	: TextStyle(InStyle)
{
	static const FSlateBrush EmptyBrush = FSlateNoResource{};
	TextStyle.SetUnderlineBrush(EmptyBrush);
	TextStyle.SetStrikeBrush(EmptyBrush);
}

TSharedRef<IRun> FText3DLayout::CreateDefaultTextRun(
	const TSharedRef<FString>& NewText,
	const FTextRange& NewRange) const
{
	return FSlateTextRun::Create(FRunInfo(), NewText, TextStyle, NewRange);
}