// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/TextLayout.h"
#include "Styling/SlateTypes.h"

/** An implementation of FTextLayout, which discards most Slate widget specific functionality. */
class FText3DLayout final : public FTextLayout
{
public:
	/** Optionally provide a custom TextBlock style. */
	explicit FText3DLayout(const FTextBlockStyle& InStyle = FTextBlockStyle::GetDefault());

	const FTextBlockStyle& GetTextStyle() const
	{
		return TextStyle;
	}

protected:
	/** Parameters relevant to text layout. */
	FTextBlockStyle TextStyle;

	/** Required but unused override. */
	virtual TSharedRef<IRun> CreateDefaultTextRun(
		const TSharedRef<FString>& NewText,
		const FTextRange& NewRange) const override;
};