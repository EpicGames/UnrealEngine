// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutBuilders/Text3DLayoutShaper.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Fonts/FontCache.h"
#include "Fonts/SlateTextShaper.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Framework/Text/ShapedTextCache.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/TextLayout.h"
#include "Internationalization/Text.h"
#include "Styling/SlateStyle.h"
#include "Text3DShapedGlyphLine.h"

TSharedPtr<FText3DLayoutShaper> FText3DLayoutShaper::Get()
{
	static TSharedPtr<FText3DLayoutShaper> Instance = MakeShared<FText3DLayoutShaper>(FPrivateToken{});
	return Instance;
}

void FText3DLayoutShaper::ShapeBidirectionalText(
	const TSharedPtr<FSlateStyleSet>& InStyles,
	const FTextBlockStyle& InDefaultStyle,
	const FString& Text,
	const TSharedPtr<FTextLayout>& TextLayout,
	const TSharedPtr<FRichTextLayoutMarshaller>& TextMarshaller,
	TArray<UE::Text3D::Layout::FGlyphLine>& OutShapedLines)
{
	using namespace UE::Text3D::Layout;

	// @todo: Restore when dependency on SlateApplication is removed. Currently this means text meshes won't be created on the server.
	// @see: UE-211843
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	checkf(InStyles.IsValid(), TEXT("Styles set is not valid, cannot proceed"))

	TextLayout->ClearLines();
	TextLayout->ClearLineHighlights();
	TextLayout->ClearRunRenderers();

	TextMarshaller->SetDecoratorStyleSet(InStyles.Get());
	TextMarshaller->SetText(Text, *TextLayout);
	TextMarshaller->ClearDirty();

	TextLayout->UpdateLayout();

	TArray<FTextLayout::FLineView> LineViews = TextLayout->GetLineViews();

	// @note: mimics FSlateTextLayout::OnPaint
	for (const FTextLayout::FLineView& Line : LineViews)
	{
		FGlyphLine& ShapedLine = OutShapedLines.AddDefaulted_GetRef();
		ShapedLine.TextDirection = Line.TextBaseDirection;

		for (const TSharedRef<ILayoutBlock>& Block : Line.Blocks)
		{
			const FTextRange RunRange = Block->GetRun()->GetTextRange();
			const FTextRange BlockRange = Block->GetTextRange();
			FString BlockText;
			Block->GetRun()->AppendTextTo(BlockText);
	
			const FLayoutBlockTextContext BlockTextContext = Block->GetTextContext();
			const FName StyleTag(Block->GetRun()->GetRunInfo().Name);
			const FSlateFontInfo& FontInfo = !StyleTag.IsNone() && InStyles->HasWidgetStyle<FTextBlockStyle>(StyleTag) ? InStyles->GetWidgetStyle<FTextBlockStyle>(StyleTag).Font : InDefaultStyle.Font;

			const FShapedGlyphSequenceRef ShapedGlyphSequence = ShapedTextCacheUtil::GetShapedTextSubSequence(
				BlockTextContext.ShapedTextCache,
				FCachedShapedTextKey(RunRange, 1.0f, BlockTextContext, FontInfo),
				BlockRange,
				*BlockText,
				BlockTextContext.TextDirection
			);

			for (const FShapedGlyphEntry& ShapedGlyphEntry : ShapedGlyphSequence->GetGlyphsToRender())
			{
				FGlyphEntry& GlyphEntry = ShapedLine.Glyphs.Add_GetRef(ShapedGlyphEntry);
				GlyphEntry.StyleTag = StyleTag;
				GlyphEntry.Size = FontInfo.Size;

#if WITH_FREETYPE
				// Font face data like metrics might change after this.
				// Metrics that will be used in stages like Layout Pre Renderer update need to be cached ahead of time.
				// In this case only the ascender and descender from size metrics are used.
				// For now, deal with storing this data with the glyph as it's only two longs (FT_Pos).
				// If more data is required later, these metrics could be stored per face, rather than per glyph.
				if (ShapedGlyphEntry.FontFaceData.IsValid())
				{
					const TSharedPtr<FFreeTypeFace> FontFace = ShapedGlyphEntry.FontFaceData->FontFace.Pin();
					if (FontFace.IsValid() && FontFace->GetFace())
					{
						const FT_Size_Metrics& SizeMetrics = FontFace->GetFace()->size->metrics;
						GlyphEntry.FaceSizeMetricsAscender = SizeMetrics.ascender;
						GlyphEntry.FaceSizeMetricsDescender = SizeMetrics.descender;
					}
				}
#endif
			}
		}
	}
}
	