// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FRichTextLayoutMarshaller;
class FSlateStyleSet;
class FTextLayout;
struct FShapedGlyphLine;
struct FTextBlockStyle;

namespace UE::Text3D::Layout
{
	struct FGlyphLine;
}

/** A singleton that handles shaping operations and writes the result to a provided TextLayout. */
class FText3DLayoutShaper final
{
public:
	/** Returns TextShaper singleton. */
	static TSharedPtr<FText3DLayoutShaper> Get();

	/**
	 * Arranges the provided text to match the requested layout, accounting for scale, offsets etc.
	 * Analogous to FSlateFontCache::ShapeBidirectionalText.
	 */
	void ShapeBidirectionalText(
		const TSharedPtr<FSlateStyleSet>& InStyles,
		const FTextBlockStyle& InDefaultStyle,
		const FString& Text,
		const TSharedPtr<FTextLayout>& TextLayout,
		const TSharedPtr<FRichTextLayoutMarshaller>& TextMarshaller,
		TArray<UE::Text3D::Layout::FGlyphLine>& OutShapedLines);

private:
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	explicit FText3DLayoutShaper(FPrivateToken)
	{
	}

private:
	friend class UText3DComponent;

	FText3DLayoutShaper(const FText3DLayoutShaper&) = delete;
	FText3DLayoutShaper& operator=(const FText3DLayoutShaper&) = delete;
};
