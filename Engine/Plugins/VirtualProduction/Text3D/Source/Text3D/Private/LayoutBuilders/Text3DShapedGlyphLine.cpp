// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutBuilders/Text3DShapedGlyphLine.h"

float UE::Text3D::Layout::FGlyphLine::GetWidthAdvance(const int32 InIndex, const float InKerning, const float InWordSpacing) const
{
	check(InIndex >= 0 && InIndex < Glyphs.Num());

	const FGlyphEntry& Glyph = Glyphs[InIndex];
	float Advance = (Glyph.Entry.XAdvance != 0) ? Glyph.Entry.XOffset + Glyph.Entry.XAdvance : Glyph.Entry.XAdvance;

	const bool bFirstCharacter = InIndex == 0;
	const bool bLastCharacter = InIndex == Glyphs.Num() - 1;
	const bool bSkipCharacter = TextDirection == TextBiDi::ETextDirection::RightToLeft ? bLastCharacter : bFirstCharacter;

	if (!bSkipCharacter)
	{
		// @note: as per FSlateElementBatcher::BuildShapedTextSequence, per Glyph Kerning isn't used
		Advance += (Glyph.Entry.XAdvance != 0) ? InKerning : 0.f;

		if (!Glyph.Entry.bIsVisible)
		{
			Advance += InWordSpacing;
		}
	}

	return Advance;
}
