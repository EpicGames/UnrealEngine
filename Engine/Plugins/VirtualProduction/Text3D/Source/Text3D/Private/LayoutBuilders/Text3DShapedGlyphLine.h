// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Fonts/FontCache.h"
#include "Text3DTypes.h"

#if WITH_FREETYPE
#include "Fonts/FontCacheFreeType.h"
#endif

namespace UE::Text3D::Layout
{
	/** Contains text glyph information for a single character */
	struct FGlyphEntry
	{
		FGlyphEntry() = default;
		FGlyphEntry(const FShapedGlyphEntry& InEntry)
			: Entry(InEntry)
		{
		}

		/** Style applied on this entry */
		FName StyleTag = NAME_None;

		/** Actual size of the glyph */
		float Size = 48.f;

		/** Actual entry for the glyph */
		FShapedGlyphEntry Entry;

#if WITH_FREETYPE
		/** 
		 * Cached metrics when the glyph is shaped. 
		 * In between Text3D incremental build phases (specifically between Layout Prepare Build and Layout Pre Renderer Update), 
		 * size metrics might change for an external reason (e.g. a text block widget with the same font).
		 */
		FT_Pos FaceSizeMetricsAscender = 0;
		FT_Pos FaceSizeMetricsDescender = 0;
#endif
	};

	/** Contains text line metric with sufficient information to fetch and transform each character. */
	struct FGlyphLine
	{
		/** Get's the offset from the previous character, accounting for kerning and word spacing. */
		float GetWidthAdvance(const int32 InIndex, const float InKerning, const float InWordSpacing) const;

		/** The corresponding shaped glyph for each character in this line of text. */
		TArray<FGlyphEntry> Glyphs;

		/** Stored result of line width */
		float Width = 0.0f;

		/** Glyph advance on this line */
		TArray<float> GlyphAdvances;

		/** Direction of the text in the line */
		TextBiDi::ETextDirection TextDirection = TextBiDi::ETextDirection::LeftToRight;

		/** Max font height for this line calculated based on MaxFontAscender and MaxFontDescender */
		float MaxFontHeight = 0.f;

		/** Positive value representing max height above baseline */
		float MaxFontAscender = 0.f;

		/** Positive value representing max height below baseline */
		float MaxFontDescender = 0.f;
	};
}