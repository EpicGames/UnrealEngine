// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutBuilders/Text3DShapedGlyphText.h"
#include "Misc/EnumerateRange.h"

#if WITH_FREETYPE
#include "Fonts/SlateTextShaper.h"
#endif

namespace UE::Text3D::Layout
{
	bool FGlyphText::IsValidGlyph(const FGlyphEntry& InGlyph)
	{
	#if WITH_FREETYPE
		return InGlyph.Entry.HasValidGlyph() && InGlyph.Entry.bIsVisible && InGlyph.Entry.GlyphIndex != 0 && InGlyph.Entry.FontFaceData.IsValid() && InGlyph.Entry.FontFaceData->FontFace.IsValid();
	#else
		return false;
	#endif
	}

	FGlyphText::FGlyphText()
	{
		Reset();
	}

	void FGlyphText::Reset()
	{
		Kernings.Reset();
		FontFaces.Reset();
		Tracking = 0.f;
		WordSpacing = 0.0f;
		bWrap = false;
		Lines.Reset();
	}

	void FGlyphText::CalculateWidth()
	{
		TArray<FGlyphLine> NewLines;
	    NewLines.Reserve(Lines.Num());

	    int32 CharacterIndex = 0;
	    for (const TConstEnumerateRef<FGlyphLine> GlyphLine : EnumerateRange(Lines))
	    {
	        FGlyphLine* CurrentLine = &NewLines.AddDefaulted_GetRef();
	        CurrentLine->TextDirection = GlyphLine->TextDirection;

	        const bool bIsRTL = (GlyphLine->TextDirection == TextBiDi::ETextDirection::RightToLeft);

	        TArray<float> CurrentAdvances;
	        TArray<FGlyphEntry> CurrentWord;
	        float LineWidth = 0.0f;
	        float CurrentWordLength = 0.0f;
    		float MaxFontAscender = 0.0f;
    		float MaxFontDescender = 0.0f;

	        const int32 GlyphCount = GlyphLine->Glyphs.Num();
	    	
	    	// Used to avoid an array copy. Here we need to iterate glyphs in logical order instead of visual order...
	    	auto ForEachGlyph = [&GlyphLine, &bIsRTL](TFunctionRef<void(const FGlyphEntry& InEntry, int32 InGlyphIndex, int32 InIterationCount)> InFunctor)
	    		{
	    			int32 ProcessedCount = 0;
	    			if (bIsRTL)
	    			{
	    				for (int32 GlyphIndex = GlyphLine->Glyphs.Num() - 1; GlyphIndex >= 0; GlyphIndex--)
	    				{
	    					InFunctor(GlyphLine->Glyphs[GlyphIndex], GlyphIndex, ProcessedCount++);
	    				}
	    			}
				    else
				    {
				    	for (int32 GlyphIndex = 0; GlyphIndex < GlyphLine->Glyphs.Num(); GlyphIndex++)
				    	{
				    		InFunctor(GlyphLine->Glyphs[GlyphIndex], GlyphIndex, ProcessedCount++);
				    	}
				    }
	    		};

	    	ForEachGlyph(
	    		[this, &CharacterIndex, &GlyphCount, &GlyphLine, &LineWidth, &CurrentWordLength, &CurrentLine, &MaxFontAscender, &MaxFontDescender, &NewLines, &CurrentWord, &CurrentAdvances, &bIsRTL]
	    		(const FGlyphEntry& CurrentGlyph, int32 GlyphIndex, int32 IterationCount)
	    		{
	    			const bool bVisibleGlyph = IsValidGlyph(CurrentGlyph);
					const int32 CharIndex = bIsRTL ? Kernings.Num() - 1 - CharacterIndex : CharacterIndex;
    				const float CurrentKerning = bVisibleGlyph ? Kernings[CharIndex] : 0.f;
    				const float GlyphAdv = GlyphLine->GetWidthAdvance(GlyphIndex, Tracking + CurrentKerning, WordSpacing);

		            // If we're at the end the line or at whitespace
		            const bool bWordBreak = !bVisibleGlyph || IterationCount == GlyphCount - 1;

    				if (bWrap                               // when we're wrapping
						&& bWordBreak                       // and at a word break
						&& LineWidth > MaxWidth             // and the current line is longer than the max
						&& CurrentWordLength != LineWidth)  // and the line is not just a single word that we can't break
    				{
    					CurrentLine->Width = LineWidth - CurrentWordLength;
    					CurrentLine->MaxFontHeight = MaxFontAscender + MaxFontDescender;
    					CurrentLine->MaxFontAscender = MaxFontAscender;
    					CurrentLine->MaxFontDescender = MaxFontDescender;

    					// Reset
    					MaxFontAscender = 0.0f;
    					MaxFontDescender = 0.0f;
    					
    					if (bIsRTL)
    					{
    						Algo::Reverse(CurrentLine->Glyphs);
    						Algo::Reverse(CurrentLine->GlyphAdvances);
    					}

    					CurrentLine = &NewLines.AddDefaulted_GetRef();
    					CurrentLine->TextDirection = GlyphLine->TextDirection;
    					LineWidth = CurrentWordLength;
    				}

					LineWidth += GlyphAdv;
					CurrentWordLength += GlyphAdv;

					if (bVisibleGlyph)
					{
#if WITH_FREETYPE
						const float Ascender = FMath::Max(0.0f, CurrentGlyph.FaceSizeMetricsAscender * UE::Text3D::Metrics::Convert26Dot6ToPixel);
						const float Descender = FMath::Max(0.0f, -CurrentGlyph.FaceSizeMetricsDescender * UE::Text3D::Metrics::Convert26Dot6ToPixel);

						MaxFontAscender  = FMath::Max(MaxFontAscender,  Ascender);
						MaxFontDescender = FMath::Max(MaxFontDescender, Descender);
#endif
					}

					CurrentWord.Add(CurrentGlyph);
					CurrentAdvances.Add(GlyphAdv);

		            if (bWordBreak)
		            {
	            		CurrentLine->Glyphs.Append(CurrentWord);
	            		CurrentLine->GlyphAdvances.Append(CurrentAdvances);
		                CurrentWordLength = 0.0f;
		                CurrentWord.Reset();
		                CurrentAdvances.Reset();
		            }

    				if (bVisibleGlyph)
    				{
    					CharacterIndex++;
    				}
	    		}
	    	);

	        CurrentLine->Width = LineWidth;
    		CurrentLine->MaxFontHeight = MaxFontAscender + MaxFontDescender;
    		CurrentLine->MaxFontAscender = MaxFontAscender;
    		CurrentLine->MaxFontDescender = MaxFontDescender;
	    	
	    	if (bIsRTL)
	    	{
	    		Algo::Reverse(CurrentLine->Glyphs);
	    		Algo::Reverse(CurrentLine->GlyphAdvances);
	    	}
	    }

	    Lines = NewLines;
	}
}