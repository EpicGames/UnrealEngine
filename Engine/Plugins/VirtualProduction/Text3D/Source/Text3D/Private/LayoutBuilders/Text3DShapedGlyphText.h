// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Text3DShapedGlyphLine.h"

class FFreeTypeFace;
struct FShapedGlyphLine;

namespace UE::Text3D::Layout
{
	/** Contains text metrics with sufficient information to fetch and transform each line. */
	struct FGlyphText
	{
		/** Check if a glyph is valid and visible (can be rendered) */
		static bool IsValidGlyph(const FGlyphEntry& InGlyph);

		FGlyphText();

		void Reset();
		void CalculateWidth();

		/** Individual character kerning */
		TArray<float> Kernings;

		/** Individual character font faces */
		TArray<TSharedPtr<FFreeTypeFace>> FontFaces;

		/** General tracking for all characters */
		float Tracking;

		/** Spacing between words */
		float WordSpacing;

		/** Maximum width allowed */
		float MaxWidth;

		/** Whether to wrap words */
		bool bWrap;

		/** Lines calculated based on current text */
		TArray<FGlyphLine> Lines;
	};
}