// Copyright Epic Games, Inc. All Rights Reserved.

#include "Renderers/DynamicMesh/Text3DDynamicMeshRenderer.h"

#include "Characters/Text3DCharacterBase.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "Engine/CollisionProfile.h"
#include "Extensions/Text3DGeometryExtensionBase.h"
#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "Extensions/Text3DRenderingExtensionBase.h"
#include "Logs/Text3DLogs.h"
#include "Subsystems/Text3DEngineSubsystem.h"
#include "Text3DComponent.h"
#include "Text3DInternalTypes.h"
#include "UObject/Package.h"

UText3DDynamicMeshRenderer::FScopedMeshEditor::~FScopedMeshEditor()
{
	UDynamicMeshComponent* const MeshComponent = Owner->DynamicMeshComponent;
	if (UpdateType != EMeshEditorUpdateType::None && IsValid(MeshComponent))
	{
		if (UpdateType == EMeshEditorUpdateType::Fast)
		{
			MeshComponent->NotifyMeshVertexAttributesModified();
		}
		else if (UpdateType == EMeshEditorUpdateType::Full)
		{
			MeshComponent->NotifyMeshUpdated();
		}

		MeshComponent->OnMeshChanged.Broadcast();
	}
}

void UText3DDynamicMeshRenderer::FScopedMeshEditor::EditMesh(const TFunctionRef<void(UE::Geometry::FDynamicMesh3&)>& InFunctor, EMeshEditorUpdateType InType)
{
	UDynamicMeshComponent* const MeshComponent = Owner->DynamicMeshComponent;
	if (MeshComponent && MeshComponent->GetDynamicMesh())
	{
		constexpr bool bDeferChanges = true;
		MeshComponent->GetDynamicMesh()->EditMesh(InFunctor, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, bDeferChanges);
		UpdateType = InType > UpdateType ? InType : UpdateType;
	}
}

void UText3DDynamicMeshRenderer::FScopedMeshEditor::ProcessMesh(const TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)>& InFunctor) const
{
	UDynamicMeshComponent* const MeshComponent = Owner->DynamicMeshComponent;
	if (MeshComponent && MeshComponent->GetDynamicMesh())
	{
		MeshComponent->GetDynamicMesh()->ProcessMesh(InFunctor);
	}
}

void UText3DDynamicMeshRenderer::OnCreate()
{
	UText3DComponent* TextComponent = GetText3DComponent();
	AActor* Owner = TextComponent->GetOwner();
	const FAttachmentTransformRules AttachRule = FAttachmentTransformRules::SnapToTargetIncludingScale;

	if (!IsValid(DynamicMeshComponent))
	{
		DynamicMeshComponent = NewObject<UDynamicMeshComponent>(this, TEXT("Text3DDynamicMesh"));
		DynamicMeshComponent->CreationMethod = EComponentCreationMethod::Instance;
		DynamicMeshComponent->RegisterComponent();
		Owner->AddOwnedComponent(DynamicMeshComponent);
		DynamicMeshComponent->AttachToComponent(TextComponent, AttachRule);

		// Setup collision
		DynamicMeshComponent->SetComplexAsSimpleCollisionEnabled(true);
		DynamicMeshComponent->SetDeferredCollisionUpdatesEnabled(true);
		DynamicMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
		DynamicMeshComponent->SetMeshDrawPath(EDynamicMeshDrawPath::StaticDraw);

		ClearMesh();
	}
	else
	{
		FDetachmentTransformRules DetachRule = FDetachmentTransformRules::KeepRelativeTransform;
		DetachRule.bCallModify = false;
		DynamicMeshComponent->DetachFromComponent(DetachRule);
		DynamicMeshComponent->AttachToComponent(TextComponent, AttachRule);

		// Ensure component is registered
		if (!DynamicMeshComponent->IsRegistered())
		{
			DynamicMeshComponent->RegisterComponent();
		}
	}
}

void UText3DDynamicMeshRenderer::OnUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	if (!DynamicMeshComponent)
	{
		UE_LOGF(LogText3D, Warning, "Invalid dynamic mesh component for %ls renderer, cannot proceed!", *GetFriendlyName().ToString())
		return;
	}

	const UText3DComponent* TextComponent = GetText3DComponent();

	if (!ScopedMeshEditor.IsSet())
	{
		ScopedMeshEditor = FScopedMeshEditor(this);

		// Restore checkpoint in case mesh was altered externally
		RestoreMesh(ScopedMeshEditor.GetValue());
	}

	FScopedMeshEditor& MeshEditor = ScopedMeshEditor.GetValue();

	if (InParameters.CurrentFlag == EText3DRendererFlags::Geometry)
	{
		UText3DGeometryExtensionBase* const GeometryExtension = TextComponent->GetGeometryExtension();
		DynamicMeshComponent->bAlwaysCreatePhysicsState = false;
		DynamicMeshComponent->SetCollisionEnabled(GeometryExtension ? GeometryExtension->GetGlyphCollisionEnabled() : ECollisionEnabled::QueryAndPhysics);

		AllocateGlyphMeshData(MeshEditor, TextComponent->GetCharacterCount());
		
		TextComponent->ForEachCharacter([this, &MeshEditor](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			if (const FText3DCachedMesh* CachedGlyph = InCharacter->GetGlyphMesh())
			{
				AppendGlyphMesh(MeshEditor, InIndex, InCharacter->GetGlyphIndex(), CachedGlyph->DynamicMesh);
			}
			else
			{
				ClearGlyphMesh(MeshEditor, InIndex);
			}
		});

		MeshEditor.ProcessMesh(
			[this](const UE::Geometry::FDynamicMesh3& InMesh)
			{
				OriginalMesh = InMesh;
			});

		RefreshBounds();
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Layout)
	{
		const UText3DLayoutExtensionBase* LayoutExtension = TextComponent->GetLayoutExtension();

		DynamicMeshComponent->SetRelativeScale3D(LayoutExtension->GetTextScale());

		TextComponent->ForEachCharacter([this, &MeshEditor](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			constexpr bool bReset = false;
			const FTransform& CharacterTransform = InCharacter->GetTransform(bReset);
			SetGlyphMeshTransform(MeshEditor, InIndex, CharacterTransform, InCharacter->GetVisibility());
		});

		RefreshBounds();
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Material)
	{
		using namespace UE::Text3D::Material;

		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		TextComponent->ForEachCharacter([this, MaterialExtension](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			if (!GlyphMeshData.IsValidIndex(InIndex))
			{
				return;
			}
			for (const int32 MaterialSlot : GlyphMeshData[InIndex].MaterialSlots)
			{
				const FName StyleTag = InCharacter->GetStyleTag();
				const EText3DGroupType GroupType = static_cast<EText3DGroupType>(MaterialSlot - (InIndex * GetSlotNames().Num()));

				FMaterialParameters Parameters;
				Parameters.Group = GroupType;
				Parameters.Tag = StyleTag;

				UMaterialInterface* Material = MaterialExtension->GetMaterial(Parameters);

				if (Material != DynamicMeshComponent->GetMaterial(MaterialSlot))
				{
					DynamicMeshComponent->SetMaterial(MaterialSlot, Material);
				}
			}
		});
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Visibility)
	{
		const UText3DRenderingExtensionBase* RenderingExtension = TextComponent->GetRenderingExtension();

		DynamicMeshComponent->SetHiddenInGame(TextComponent->bHiddenInGame);
		DynamicMeshComponent->SetCastShadow(RenderingExtension->GetTextCastShadow());
		DynamicMeshComponent->SetCastHiddenShadow(RenderingExtension->GetTextCastHiddenShadow());
		DynamicMeshComponent->SetAffectDynamicIndirectLighting(RenderingExtension->GetTextAffectDynamicIndirectLighting());
		DynamicMeshComponent->SetAffectIndirectLightingWhileHidden(RenderingExtension->GetTextAffectIndirectLightingWhileHidden());
		DynamicMeshComponent->SetHoldout(RenderingExtension->GetTextHoldout());
		DynamicMeshComponent->SetVisibility(TextComponent->GetVisibleFlag());

		TextComponent->ForEachCharacter([this, &MeshEditor](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			constexpr bool bReset = false;
			const FTransform& CharacterTransform = InCharacter->GetTransform(bReset);
			SetGlyphMeshTransform(MeshEditor, InIndex, CharacterTransform, InCharacter->GetVisibility());
		});
	}

	if (InParameters.bIsLastFlag)
	{
		// Save checkpoint to restore mesh on next update
		SaveMesh(MeshEditor);

		ScopedMeshEditor.Reset();
	}
}

void UText3DDynamicMeshRenderer::OnClear()
{
	if (DynamicMeshComponent)
	{
		DynamicMeshComponent->EditMesh([](FDynamicMesh3& InEditMesh)
		{
			for (const int32 TId : InEditMesh.TriangleIndicesItr())
			{
				InEditMesh.RemoveTriangle(TId);
			}
		});
	}
}

void UText3DDynamicMeshRenderer::OnDestroy()
{
	if (DynamicMeshComponent)
	{
		DynamicMeshComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		DynamicMeshComponent->DestroyComponent();
	}
}

EText3DMeshType UText3DDynamicMeshRenderer::GetMeshType() const
{
	return EText3DMeshType::Dynamic;
}

FName UText3DDynamicMeshRenderer::GetFriendlyName() const
{
	static const FName Name(TEXT("DynamicMeshRenderer"));
	return Name;
}

FBox UText3DDynamicMeshRenderer::OnCalculateBounds() const
{
	FBox Box(ForceInit);
	Box += DynamicMeshComponent->Bounds.GetBox();
	return Box;
}

void UText3DDynamicMeshRenderer::OnIterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const
{
	if (DynamicMeshComponent)
	{
		InFunc(DynamicMeshComponent);
	}
}

#if WITH_EDITOR
void UText3DDynamicMeshRenderer::OnDebugModeEnabled()
{
	Super::OnDebugModeEnabled();
	SetDebugMode(true);
}

void UText3DDynamicMeshRenderer::OnDebugModeDisabled()
{
	Super::OnDebugModeDisabled();
	SetDebugMode(false);
}
#endif

void UText3DDynamicMeshRenderer::PostRename(UObject* InOldOuter, const FName InOldName)
{
	Super::PostRename(InOldOuter, InOldName);

	auto RenameComponent = [InOldOuter, this](UActorComponent* InComponent)
		{
			if (InComponent)
			{
				// Force trigger PostRename on the components as its one of the few places where the cached owner is updated
				InComponent->PostRename(InOldOuter, InComponent->GetFName());
			}
		};

	RenameComponent(DynamicMeshComponent);
}

void UText3DDynamicMeshRenderer::BeginDestroy()
{
	Super::BeginDestroy();
	ScopedMeshEditor.Reset();
}

#if WITH_EDITOR
void UText3DDynamicMeshRenderer::SetDebugMode(bool bInEnabled)
{
	// Since we are dealing with class FProperty, no need to run this for each instance, do it once
	static bool bDebugModeEnabled = true;

	if (bDebugModeEnabled != bInEnabled)
	{
		bDebugModeEnabled = bInEnabled;

		FProperty* DynamicMeshComponentProperty = FindFProperty<FProperty>(UText3DDynamicMeshRenderer::StaticClass(), GET_MEMBER_NAME_CHECKED(UText3DDynamicMeshRenderer, DynamicMeshComponent));

		// Here we toggle the CPF_Edit flag to hide/show property in the details panel component editor tree / outliner
		// @see FComponentEditorUtils::GetPropertyForEditableNativeComponent
		// todo : implement a custom debug view widget for text or add a property editor metadata to control the component visibility in the component editor tree / outliner
		if (bInEnabled)
		{
			DynamicMeshComponentProperty->SetPropertyFlags(CPF_Edit);
		}
		else
		{
			DynamicMeshComponentProperty->ClearPropertyFlags(CPF_Edit);
		}
	}
}
#endif

void UText3DDynamicMeshRenderer::AllocateGlyphMeshData(FScopedMeshEditor& InMeshEditor, int32 InCount)
{
	if (GlyphMeshData.Num() > InCount)
	{
		for (int32 Index = InCount; Index < GlyphMeshData.Num(); Index++)
		{
			ClearGlyphMesh(InMeshEditor, Index);
		}
	}

	GlyphMeshData.SetNum(InCount);
}

void UText3DDynamicMeshRenderer::AppendGlyphMesh(FScopedMeshEditor& InMeshEditor, uint16 InIndex, uint32 InGlyphIndex, UDynamicMesh* InMesh)
{
	using namespace UE::Geometry;

	check(!!InMesh);

	if (!GlyphMeshData.IsValidIndex(InIndex))
	{
		return;
	}

	FBox Bounds = FBox(ForceInitToZero);
	InMesh->ProcessMesh([&Bounds](const FDynamicMesh3& InMesh)
	{
		Bounds = static_cast<FBox>(InMesh.GetBounds(true));
	});

	if (GlyphMeshData[InIndex].GlyphIndex == InGlyphIndex && GlyphMeshData[InIndex].Bounds.Equals(Bounds))
	{
		return;
	}

	ClearGlyphMesh(InMeshEditor, InIndex);

	GlyphMeshData[InIndex].GlyphIndex = InGlyphIndex;
	GlyphMeshData[InIndex].Bounds = Bounds;
	GlyphMeshData[InIndex].CurrentTransform = FTransform::Identity;

	InMeshEditor.EditMesh(
		[this, InMesh, InIndex, InGlyphIndex](FDynamicMesh3& InEditMesh)
		{
			InMesh->ProcessMesh([this, &InEditMesh, InIndex, InGlyphIndex](const FDynamicMesh3& InProcessMesh)
			{
				FMeshIndexMappings Mappings;
				FDynamicMeshEditor Editor(&InEditMesh);
				Editor.AppendMesh(&InProcessMesh, Mappings);

				TArray<int32> NewTriangles;
				const TMap<int32, int32>& FromToMap = Mappings.GetTriangleMap().GetForwardMap();
				FDynamicMeshMaterialAttribute* MaterialAttribute = InEditMesh.Attributes()->GetMaterialID();

				TArray<int32> MaterialSlots;
				for (const TPair<int32, int32>& FromToPair : FromToMap)
				{
					InEditMesh.SetTriangleGroup(FromToPair.Value, InIndex);
					const int32 MaterialID = (InIndex * UE::Text3D::Material::GetSlotNames().Num()) + MaterialAttribute->GetValue(FromToPair.Value);
					MaterialAttribute->SetValue(FromToPair.Value, MaterialID);
					MaterialSlots.AddUnique(MaterialID);
				}

				MaterialSlots.StableSort([](int32 A, int32 B)
				{
					return A > B;
				});

				GlyphMeshData[InIndex].MaterialSlots = MoveTemp(MaterialSlots);
			});
		}, EMeshEditorUpdateType::Full);
}

void UText3DDynamicMeshRenderer::ClearMesh()
{
	GlyphMeshData.Reset();
	DynamicMeshComponent->EditMesh([](FDynamicMesh3& InEditMesh)
	{
		InEditMesh.Clear();
		InEditMesh.EnableTriangleGroups();
		InEditMesh.EnableAttributes();
		InEditMesh.Attributes()->EnableMaterialID();
		InEditMesh.Attributes()->EnableTangents();
		InEditMesh.Attributes()->SetNumUVLayers(1);
	});
}

void UText3DDynamicMeshRenderer::ClearGlyphMesh(FScopedMeshEditor& InMeshEditor, uint16 InIndex)
{
	using namespace UE::Geometry;

	// Remove old glyph data
	InMeshEditor.EditMesh(
		[this, InIndex](FDynamicMesh3& InEditMesh)
		{
			TArray<int32> TrianglesToDelete;
			for (const int32 Tid : InEditMesh.TriangleIndicesItr())
			{
				if (InEditMesh.GetTriangleGroup(Tid) == InIndex)
				{
					TrianglesToDelete.Add(Tid);
				}
			}

			FDynamicMeshEditor Editor(&InEditMesh);
			Editor.RemoveTriangles(TrianglesToDelete, /** RemoveIsolatedVertices*/ true);
			InEditMesh.CompactInPlace();
		}, EMeshEditorUpdateType::Full);
}

void UText3DDynamicMeshRenderer::SetGlyphMeshTransform(FScopedMeshEditor& InMeshEditor, uint16 InIndex, const FTransform& InTransform, bool bInVisible)
{
	using namespace UE::Geometry;

	if (!GlyphMeshData.IsValidIndex(InIndex))
	{
		return;
	}

	FGlyphMeshData& GlyphData = GlyphMeshData[InIndex];

	if (!GlyphData.bVisible && GlyphData.bVisible == bInVisible)
	{
		return;
	}

	GlyphData.bVisible = bInVisible;

	const FVector TargetScale = GlyphData.bVisible ? InTransform.GetScale3D().ComponentMax(FVector(UE_KINDA_SMALL_NUMBER)) : FVector(UE_KINDA_SMALL_NUMBER);
	const FTransform TargetTransform(InTransform.GetRotation(), InTransform.GetTranslation(), TargetScale);

	if (GlyphData.CurrentTransform.Equals(TargetTransform))
	{
		return;
	}

	const bool bUseOriginalMesh = OriginalMesh.IsSet();

	const FTransform3d InverseTransform = bUseOriginalMesh ? FTransform3d::Identity : GlyphData.CurrentTransform.Inverse();
	const FQuat DeltaRotation = bUseOriginalMesh ? TargetTransform.GetRotation() : TargetTransform.GetRotation() * GlyphData.CurrentTransform.GetRotation().Inverse();

	InMeshEditor.EditMesh([InIndex, &GlyphData, &InverseTransform, &DeltaRotation, &TargetTransform, &OriginalMesh=OriginalMesh, bUseOriginalMesh](FDynamicMesh3& InEditMesh)
	{
		const FDynamicMesh3& ReferenceMesh = bUseOriginalMesh ? *OriginalMesh : InEditMesh;

		FDynamicMeshNormalOverlay* Normals = InEditMesh.Attributes()->PrimaryNormals();
		FDynamicMeshNormalOverlay* Tangents = InEditMesh.Attributes()->PrimaryTangents();

		const FDynamicMeshNormalOverlay* ReferenceNormals = ReferenceMesh.Attributes()->PrimaryNormals();
		const FDynamicMeshNormalOverlay* ReferenceTangents = ReferenceMesh.Attributes()->PrimaryTangents();

		TSet<int32> VerticesToTransform;
		TSet<int32> NormalsToTransform;
		TSet<int32> TangentsToTransform;
		for (const int32 TId : InEditMesh.TriangleIndicesItr())
		{
			if (InEditMesh.GetTriangleGroup(TId) == InIndex)
			{
				FIndex3i Tri = InEditMesh.GetTriangle(TId);

				VerticesToTransform.Add(Tri.A);
				VerticesToTransform.Add(Tri.B);
				VerticesToTransform.Add(Tri.C);

				NormalsToTransform.Add(Normals->GetElementIDAtVertex(TId, Tri.A));
				NormalsToTransform.Add(Normals->GetElementIDAtVertex(TId, Tri.B));
				NormalsToTransform.Add(Normals->GetElementIDAtVertex(TId, Tri.C));

				TangentsToTransform.Add(Tangents->GetElementIDAtVertex(TId, Tri.A));
				TangentsToTransform.Add(Tangents->GetElementIDAtVertex(TId, Tri.B));
				TangentsToTransform.Add(Tangents->GetElementIDAtVertex(TId, Tri.C));
			}
		}

		for (const int32 VId : VerticesToTransform)
		{
			FVector3d LocalPos = InverseTransform.TransformPosition(ReferenceMesh.GetVertex(VId));
			FVector3d NewWorldPos = TargetTransform.TransformPosition(LocalPos);
			InEditMesh.SetVertex(VId, NewWorldPos);
		}

		for (const int32 NId : NormalsToTransform)
		{
			if (NId != FDynamicMesh3::InvalidID)
			{
				FVector3f Normal = ReferenceNormals->GetElement(NId);
				FVector3d NewNormal = DeltaRotation.RotateVector((FVector3d)Normal);
				Normals->SetElement(NId, (FVector3f)NewNormal.GetSafeNormal());
			}
		}

		for (const int32 TId : TangentsToTransform)
		{
			if (TId != FDynamicMesh3::InvalidID)
			{
				FVector3f Tangent = ReferenceTangents->GetElement(TId);
				FVector3d NewTangent = DeltaRotation.RotateVector((FVector3d)Tangent);
				Tangents->SetElement(TId, (FVector3f)NewTangent.GetSafeNormal());	
			}
		}
	}, EMeshEditorUpdateType::Fast);

	GlyphData.CurrentTransform = TargetTransform;
}

void UText3DDynamicMeshRenderer::SaveMesh(const FScopedMeshEditor& InMeshEditor)
{
	InMeshEditor.ProcessMesh([this](const FDynamicMesh3& EditMesh)
	{
		CachedMesh = EditMesh;
	});
}

void UText3DDynamicMeshRenderer::RestoreMesh(FScopedMeshEditor& InMeshEditor)
{
	if (CachedMesh.IsSet())
	{
		InMeshEditor.EditMesh([this](FDynamicMesh3& EditMesh)
		{
			EditMesh = MoveTemp(CachedMesh.GetValue());
			CachedMesh.Reset();
		}, EMeshEditorUpdateType::Fast);
	}
}
