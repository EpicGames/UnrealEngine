// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Renderers/Text3DRendererBase.h"
#include "UObject/ObjectPtr.h"
#include "Text3DDynamicMeshRenderer.generated.h"

class UDynamicMesh;
class UDynamicMeshComponent;

/**
 * Renderer that uses a single dynamic mesh component to render all characters
 * Text3DComponent
 * - Text3DDynamicMesh
 */
UCLASS(DisplayName="DynamicMeshRenderer", ClassGroup="Text3D", HideCategories=(Renderer))
class UText3DDynamicMeshRenderer : public UText3DRendererBase
{
	GENERATED_BODY()

protected:
	/** Used to store data about character glyph mesh */
	struct FGlyphMeshData
	{
		bool bVisible = true;
		uint16 GlyphIndex = 0;
		TArray<int32> MaterialSlots;
		FTransform CurrentTransform = FTransform::Identity;
		FBox Bounds = FBox(ForceInitToZero);
	};

	/** Mesh update type */
	enum class EMeshEditorUpdateType : uint8
	{
		None,
		Fast,
		Full
	};

	/** Used to push render changes when destroyed */
	struct FScopedMeshEditor
	{
		explicit FScopedMeshEditor(UText3DDynamicMeshRenderer* InComponent)
			: Owner(InComponent)
		{}

		~FScopedMeshEditor();
		void EditMesh(const TFunctionRef<void(UE::Geometry::FDynamicMesh3&)>& InFunctor, EMeshEditorUpdateType InType);
		void ProcessMesh(const TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)>& InFunctor) const;

	private:
		UText3DDynamicMeshRenderer* Owner;
		EMeshEditorUpdateType UpdateType = EMeshEditorUpdateType::None;
	};

	//~ Begin UText3DRendererBase
	virtual void OnCreate() override;
	virtual void OnUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	virtual void OnClear() override;
	virtual void OnDestroy() override;
	virtual EText3DMeshType GetMeshType() const override;
	virtual FName GetFriendlyName() const override;
	virtual FBox OnCalculateBounds() const override;
	virtual void OnIterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const override;
#if WITH_EDITOR
	virtual void OnDebugModeEnabled() override;
	virtual void OnDebugModeDisabled() override;
#endif
	//~ End UText3DRendererBase

	//~ Begin UObject
	virtual void PostRename(UObject* InOldOuter, const FName InOldName) override;
	virtual void BeginDestroy() override;
	//~ End UObject

#if WITH_EDITOR
	void SetDebugMode(bool bInEnabled);
#endif

	void AllocateGlyphMeshData(FScopedMeshEditor& InMeshEditor, int32 InCount);
	void AppendGlyphMesh(FScopedMeshEditor& InMeshEditor, uint16 InIndex, uint32 InGlyphIndex, UDynamicMesh* InMesh);
	void ClearMesh();
	void ClearGlyphMesh(FScopedMeshEditor& InMeshEditor, uint16 InIndex);
	void SetGlyphMeshTransform(FScopedMeshEditor& InMeshEditor, uint16 InIndex, const FTransform& InTransform, bool bInVisible);
	
	void SaveMesh(const FScopedMeshEditor& InMeshEditor);
	void RestoreMesh(FScopedMeshEditor& InMeshEditor);

	UPROPERTY(VisibleAnywhere, Instanced, Category="Renderer", meta=(EditCondition="false", EditConditionHides))
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;

	TArray<FGlyphMeshData> GlyphMeshData;
	TOptional<FScopedMeshEditor> ScopedMeshEditor;
	TOptional<UE::Geometry::FDynamicMesh3> CachedMesh;
	/** 
	 * Mesh generated by the geometry update. Not modified by other extensions. 
	 * Used as ground truth to avoid dealing with delta transforms (floating point inaccuracies) 
	 */
	TOptional<UE::Geometry::FDynamicMesh3> OriginalMesh;
};
