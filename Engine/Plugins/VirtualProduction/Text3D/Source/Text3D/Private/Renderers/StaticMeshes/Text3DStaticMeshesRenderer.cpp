// Copyright Epic Games, Inc. All Rights Reserved.

#include "Renderers/StaticMeshes/Text3DStaticMeshesRenderer.h"

#include "Algo/ForEach.h"
#include "Characters/Text3DCharacterBase.h"
#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Extensions/Text3DGeometryExtensionBase.h"
#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "Extensions/Text3DRenderingExtensionBase.h"
#include "Logs/Text3DLogs.h"
#include "SceneInterface.h"
#include "Settings/Text3DProjectSettings.h"
#include "Text3DComponent.h"
#include "Text3DInternalTypes.h"
#include "UObject/Package.h"
#include "Utilities/Text3DUtilities.h"

#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

#define LOCTEXT_NAMESPACE "Text3DStaticMeshesRenderer"

namespace UE::Text3D::Private
{
	static int32 RegisterStaticMeshesMode = 2;
	static FAutoConsoleVariableRef CVarRegisterStaticMeshes(
		TEXT("Text3D.RegisterStaticMeshesMode"),
		RegisterStaticMeshesMode,
		TEXT("Determines how static mesh component primitives should be batch added to the scene.\n")
		TEXT("0: Disabled. Primitives will be added one by one, not in batch.\n")
		TEXT("1: Immediate. Primitives will be added in batch to the scene as soon as all the static meshes are set.\n")
		TEXT("2: Deferred. Primitives will be added to the scene on end of frame updates.\n"),
		ECVF_Default);
}

void UText3DStaticMeshesRenderer::OnCreate()
{
	UText3DComponent* TextComponent = GetText3DComponent();
	AActor* Owner = TextComponent->GetOwner();
	const FAttachmentTransformRules AttachRule = FAttachmentTransformRules::SnapToTargetIncludingScale;
	
	if (!IsValid(TextRoot))
	{
		TextRoot = NewObject<USceneComponent>(this, TEXT("TextRoot"));
		TextRoot->CreationMethod = EComponentCreationMethod::Instance;
		TextRoot->RegisterComponent();
		Owner->AddOwnedComponent(TextRoot);
		TextRoot->AttachToComponent(TextComponent, AttachRule);
	}
	else
	{
		FDetachmentTransformRules DetachRule = FDetachmentTransformRules::KeepRelativeTransform;
		DetachRule.bCallModify = false;
		TextRoot->DetachFromComponent(DetachRule);
		TextRoot->AttachToComponent(TextComponent, AttachRule);

		// Ensure component is registered.
		// No need to do this for mesh components as they are moved to the pool (unregistered) and registered back when used (in AllocateComponents)
		if (!TextRoot->IsRegistered())
		{
			TextRoot->RegisterComponent();
		}

		if (TextRoot->GetOwner() != Owner)
		{
			// Temporarily move the root to the transient package to retrigger UActorComponent::PostRename which updates
			// OwnerPrivate only when the outers have changed.
			constexpr ERenameFlags RenameFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;
			TextRoot->Rename(nullptr, GetTransientPackage(), RenameFlags);
			TextRoot->Rename(nullptr, this, RenameFlags);
		}
	}

	AllocateComponents(0);
}

void UText3DStaticMeshesRenderer::OnUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	const UText3DComponent* TextComponent = GetText3DComponent();

	if (!TextRoot)
	{
		UE_LOGF(LogText3D, Warning, "Invalid root component for %ls renderer, cannot proceed!", *GetFriendlyName().ToString())
		return;
	}

	if (InParameters.CurrentFlag == EText3DRendererFlags::Geometry)
	{
		AllocateComponents(TextComponent->GetCharacterCount());

		UWorld* const World = TextComponent->GetWorld();

		const TArrayView<UPrimitiveComponent*> StaticMeshComponents = MakeArrayView(reinterpret_cast<UPrimitiveComponent**>(CharacterMeshes.GetData()), CharacterMeshes.Num());

		const bool bShouldBulkReregister = World 
			&& !StaticMeshComponents.IsEmpty() 
			&& UE::Text3D::Private::RegisterStaticMeshesMode > 0
			&& !StaticMeshComponents.ContainsByPredicate([](UPrimitiveComponent* InComponent)
				{
					return !InComponent || !InComponent->IsRegistered();
				});

		const bool bShouldBatchAddMeshes = bShouldBulkReregister && UE::Text3D::Private::RegisterStaticMeshesMode == 1;

		auto SetBulkReregister = [&StaticMeshComponents](bool bInNewValue)
			{
				for (UPrimitiveComponent* Component : StaticMeshComponents)
				{
					if (Component)
					{
						Component->bBulkReregister = bInNewValue;
					}
				}
			};

		if (bShouldBulkReregister)
		{
			if (bShouldBatchAddMeshes)
			{
				World->Scene->BatchRemovePrimitives(StaticMeshComponents);
				World->Scene->BatchReleasePrimitives(StaticMeshComponents);
			}
			SetBulkReregister(true);
		}

		UText3DGeometryExtensionBase* const GeometryExtension = TextComponent->GetGeometryExtension();
		const ECollisionEnabled::Type CollisionEnabled = GeometryExtension ? GeometryExtension->GetGlyphCollisionEnabled() : ECollisionEnabled::QueryAndPhysics;

		TextComponent->ForEachCharacter([this, CollisionEnabled](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			if (CharacterMeshes.IsValidIndex(InIndex))
			{
				if (UStaticMeshComponent* const StaticMeshComponent = CharacterMeshes[InIndex])
				{
					StaticMeshComponent->bAlwaysCreatePhysicsState = false;
					StaticMeshComponent->SetCollisionEnabled(CollisionEnabled);
					if (const FText3DCachedMesh* CachedMesh = InCharacter->GetGlyphMesh())
					{
						StaticMeshComponent->SetStaticMesh(CachedMesh->StaticMesh);
					}
					else
					{
						StaticMeshComponent->SetStaticMesh(nullptr);
					}
				}
				else
				{
					UE_LOGF(LogText3D, Warning, "Invalid component object for character %i", InIndex)
				}
			}
			else
			{
				UE_LOGF(LogText3D, Warning, "Invalid component index for character %i", InIndex)
			}
		});

		if (bShouldBulkReregister)
		{
			if (bShouldBatchAddMeshes)
			{
				World->Scene->BatchAddPrimitives(StaticMeshComponents);
			}
			SetBulkReregister(false);
		}
		RefreshBounds();
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Layout)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UText3DStaticMeshesRenderer::OnUpdate_Layout);

		const UText3DLayoutExtensionBase* LayoutExtension = TextComponent->GetLayoutExtension();

		TextRoot->SetRelativeScale3D(LayoutExtension->GetTextScale());

		TextComponent->ForEachCharacter([this](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			constexpr bool bReset = false;
			const FTransform& CharacterTransform = InCharacter->GetTransform(bReset);

			if (CharacterMeshes.IsValidIndex(InIndex))
			{
				if (UStaticMeshComponent* CharacterMeshComponent = CharacterMeshes[InIndex])
				{
					// Group the movement updates into one
					FScopedMovementUpdate ScopedMovementUpdate(CharacterMeshComponent);
					CharacterMeshComponent->ResetRelativeTransform();
					CharacterMeshComponent->SetRelativeTransform(CharacterTransform);
				}
				else
				{
					UE_LOGF(LogText3D, Warning, "Invalid component object for character %i", InIndex)
				}
			}
			else
			{
				UE_LOGF(LogText3D, Warning, "Invalid component index for character %i", InIndex)
			}
		});

		RefreshBounds();
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Material)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UText3DStaticMeshesRenderer::OnUpdate_Material);

		using namespace UE::Text3D::Material;

		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		TextComponent->ForEachCharacter([this, MaterialExtension](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			if (CharacterMeshes.IsValidIndex(InIndex))
			{
				if (UStaticMeshComponent* CharacterMeshComponent = CharacterMeshes[InIndex])
				{
					for (int32 GroupIndex = 0; GroupIndex < static_cast<int32>(EText3DGroupType::TypeCount); GroupIndex++)
					{
						const int32 MaterialIndex = CharacterMeshComponent->GetMaterialIndex(GetSlotNames()[GroupIndex]);

						if (MaterialIndex == INDEX_NONE)
						{
							continue;
						}

						const EText3DGroupType GroupType = static_cast<EText3DGroupType>(GroupIndex);
						const FName StyleTag = InCharacter->GetStyleTag();

						FMaterialParameters Parameters;
						Parameters.Group = GroupType;
						Parameters.Tag = StyleTag;

						UMaterialInterface* Material = MaterialExtension->GetMaterial(Parameters);

						if (Material != CharacterMeshComponent->GetMaterial(MaterialIndex))
						{
							CharacterMeshComponent->SetMaterial(MaterialIndex, Material);
						}
					}
				}
				else
				{
					UE_LOGF(LogText3D, Verbose, "Invalid component object for character %i", InIndex)
				}
			}
			else
			{
				UE_LOGF(LogText3D, Verbose, "Invalid component index for character %i", InIndex)
			}
		});
	}
	else if (InParameters.CurrentFlag == EText3DRendererFlags::Visibility)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UText3DStaticMeshesRenderer::OnUpdate_Visibility);

		const UText3DRenderingExtensionBase* RenderingExtension = TextComponent->GetRenderingExtension();

		TextComponent->ForEachCharacter([this, TextComponent, RenderingExtension](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
		{
			if (CharacterMeshes.IsValidIndex(InIndex))
			{
				if (UStaticMeshComponent* CharacterMeshComponent = CharacterMeshes[InIndex])
				{
					CharacterMeshComponent->SetHiddenInGame(TextComponent->bHiddenInGame);
					CharacterMeshComponent->SetVisibility(TextComponent->GetVisibleFlag() && InCharacter->GetVisibility());
					CharacterMeshComponent->SetCastShadow(RenderingExtension->GetTextCastShadow());
					CharacterMeshComponent->SetCastHiddenShadow(RenderingExtension->GetTextCastHiddenShadow());
					CharacterMeshComponent->SetAffectDynamicIndirectLighting(RenderingExtension->GetTextAffectDynamicIndirectLighting());
					CharacterMeshComponent->SetAffectIndirectLightingWhileHidden(RenderingExtension->GetTextAffectIndirectLightingWhileHidden());
					CharacterMeshComponent->SetHoldout(RenderingExtension->GetTextHoldout());
				}
				else
				{
					UE_LOGF(LogText3D, Warning, "Invalid component object for character %i", InIndex)
				}
			}
			else
			{
				UE_LOGF(LogText3D, Warning, "Invalid component index for character %i", InIndex)
			}
		});
	}
}

void UText3DStaticMeshesRenderer::OnClear()
{
	AllocateComponents(0);
}

void UText3DStaticMeshesRenderer::OnDestroy()
{
	AllocateComponents(0);
	CharacterMeshes.Reset();

	for (UStaticMeshComponent* MeshComponent : CharacterMeshesPool)
	{
		if (MeshComponent)
		{
			MeshComponent->DestroyComponent();
		}
	}
	CharacterMeshesPool.Reset();

	if (TextRoot)
	{
		TextRoot->DestroyComponent();
	}
}

EText3DMeshType UText3DStaticMeshesRenderer::GetMeshType() const
{
	return EText3DMeshType::Static;
}

FName UText3DStaticMeshesRenderer::GetFriendlyName() const
{
	static const FName Name(TEXT("StaticMeshesRenderer"));
	return Name;
}

FBox UText3DStaticMeshesRenderer::OnCalculateBounds() const
{
	FBox Box(ForceInit);

	for (const UStaticMeshComponent* StaticMeshComponent : CharacterMeshes)
	{
		Box += StaticMeshComponent->Bounds.GetBox();
	}

	return Box;
}

void UText3DStaticMeshesRenderer::OnIterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const
{
	for (const UStaticMeshComponent* CharacterMesh : CharacterMeshes)
	{
		if (CharacterMesh)
		{
			InFunc(CharacterMesh);
		}
	}
}

void UText3DStaticMeshesRenderer::AllocateComponents(int32 InCount)
{
	CharacterMeshes.RemoveAll([](const UStaticMeshComponent* InComponent)
		{
			return !InComponent;
		}
	);

	if (CharacterMeshes.Num() == InCount)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DStaticMeshesRenderer::AllocateComponents);

	CharacterMeshes.Reserve(InCount);

	const int32 RemainingCharacterCount = CharacterMeshes.Num() - InCount;

	const UText3DComponent* TextComponent = GetText3DComponent();
	AActor* const Actor = TextComponent->GetOwner();
	if (!Actor)
	{
		return;
	}

	// Adding to/removing from the CharacterMeshesPool doesn't set/clear the transient flags
	// so needs to be done manually here
	constexpr EObjectFlags PoolFlags = RF_Transient | RF_DuplicateTransient | RF_TextExportTransient;

	if (RemainingCharacterCount > 0)
	{
		FDetachmentTransformRules DetachmentRule = FDetachmentTransformRules::KeepRelativeTransform;
		DetachmentRule.bCallModify = false;

		for (int32 CharacterIndex = InCount; CharacterIndex < CharacterMeshes.Num(); CharacterIndex++)
		{
			if (UStaticMeshComponent* MeshComponent = CharacterMeshes[CharacterIndex])
			{
				// Prevent the component from auto registering once removed from the active meshes
				MeshComponent->bAutoRegister = false;
				MeshComponent->DetachFromComponent(DetachmentRule);
				if (MeshComponent->IsRegistered())
				{
					MeshComponent->UnregisterComponent();
				}
				Actor->RemoveOwnedComponent(MeshComponent);
				Actor->RemoveInstanceComponent(MeshComponent);

				if (MeshComponent->GetOwner() == Actor)
				{
					MeshComponent->SetFlags(PoolFlags);
					CharacterMeshesPool.Add(MeshComponent);
				}
			}
		}
	}
	else if (RemainingCharacterCount < 0)
	{
		const int32 RemainingCharacterCountPositive = FMath::Abs(RemainingCharacterCount);
		const FName CharacterMeshName = TEXT("CharacterMesh");

		UStaticMeshComponent* const Template = FindTemplate();

		for (int32 CharacterIndex = 0; CharacterIndex < RemainingCharacterCountPositive; ++CharacterIndex)
		{
			UStaticMeshComponent* MeshComponent = nullptr;
			while (!CharacterMeshesPool.IsEmpty() && !MeshComponent)
			{
				MeshComponent = CharacterMeshesPool.Pop();
			}

			if (!MeshComponent)
			{
				const FName MeshComponentName = MakeUniqueObjectName(this, UStaticMeshComponent::StaticClass(), CharacterMeshName);

				// Using DuplicateObject over NewObject with a specified template object for better support (via Pre/PostDuplicate, DuplicateTransient, etc).
				MeshComponent = Template
					? DuplicateObject<UStaticMeshComponent>(Template, this, MeshComponentName)
					: NewObject<UStaticMeshComponent>(this, MeshComponentName);

				MeshComponent->CreationMethod = EComponentCreationMethod::Instance;
			}

			if (MeshComponent)
			{
				// Re-allow the component to auto register (pooled components have this changed to off)
				MeshComponent->bAutoRegister = true;
				MeshComponent->ClearFlags(PoolFlags);
				Actor->AddOwnedComponent(MeshComponent);
				CharacterMeshes.Add(MeshComponent);
				MeshComponent->RegisterComponent();
				MeshComponent->AttachToComponent(TextRoot, FAttachmentTransformRules::SnapToTargetIncludingScale);
			}
		}
	}

	CharacterMeshes.SetNum(InCount);
}

UStaticMeshComponent* UText3DStaticMeshesRenderer::FindTemplate() const
{
	if (!GetDefault<UText3DProjectSettings>()->GetUseExistingCharactersAsTemplates())
	{
		return nullptr;
	}

	for (UStaticMeshComponent* CharacterMesh : CharacterMeshes)
	{
		if (CharacterMesh)
		{
			return CharacterMesh;
		}
	}
	// Find from pool in reverse as the first mesh to be used will be popped from the end
	for (UStaticMeshComponent* CharacterMesh : ReverseIterate(CharacterMeshesPool))
	{
		if (CharacterMesh)
		{
			return CharacterMesh;
		}
	}
	return nullptr;
}

int32 UText3DStaticMeshesRenderer::GetGlyphCount()
{
	if (!TextRoot)
	{
		return 0;
	}

	return TextRoot->GetNumChildrenComponents();
}

UStaticMeshComponent* UText3DStaticMeshesRenderer::GetGlyphMeshComponent(int32 Index)
{
	if (!CharacterMeshes.IsValidIndex(Index))
	{
		return nullptr;
	}

	return CharacterMeshes[Index];
}

const TArray<UStaticMeshComponent*>& UText3DStaticMeshesRenderer::GetGlyphMeshComponents()
{
	return CharacterMeshes;
}

#if WITH_EDITOR
void UText3DStaticMeshesRenderer::ConvertToStaticMesh()
{
	const UText3DComponent* Text3DComponent = GetText3DComponent();
	if (!IsValid(Text3DComponent) || !IsValid(Text3DComponent->GetOwner()))
	{
		return;
	}

	const AActor* Owner = Text3DComponent->GetOwner();
	FScopedSlowTask SlowTask(0.0f, LOCTEXT("ConvertToStaticMesh", "Converting Text3D to static mesh"));
	SlowTask.MakeDialog();

	if (UE::Text3D::Utilities::Conversion::ConvertToStaticMesh(Text3DComponent))
	{
		UE_LOGF(LogText3D, Log, "%ls : ConvertToStaticMesh Completed", *Owner->GetActorNameOrLabel())
	}
	else
	{
		UE_LOGF(LogText3D, Warning, "%ls : ConvertToStaticMesh Failed", *Owner->GetActorNameOrLabel())
	}
}

void UText3DStaticMeshesRenderer::OnDebugModeEnabled()
{
	Super::OnDebugModeEnabled();
	SetDebugMode(true);
}

void UText3DStaticMeshesRenderer::OnDebugModeDisabled()
{
	Super::OnDebugModeDisabled();
	SetDebugMode(false);
}
#endif

void UText3DStaticMeshesRenderer::PostRename(UObject* InOldOuter, const FName InOldName)
{
	Super::PostRename(InOldOuter, InOldName);

	// Renderer's managed components are not outered to the Text3D Component directly,
	// Use the renderer's old outer to trigger a refresh on the components cached owner.
	// The renderer's old outer gives the only remaining context to the 'old actor' (if it was outered to one), which would allow these components to be removed themselves from it.
	auto RenameComponent = [InOldOuter, this](UActorComponent* InComponent)
		{
			if (InComponent)
			{
				// Force trigger PostRename on the components as its one of the few places where the cached owner is updated
				InComponent->PostRename(InOldOuter, InComponent->GetFName());
			}
		};

	RenameComponent(TextRoot);
	Algo::ForEach(CharacterMeshes, RenameComponent);
	Algo::ForEach(CharacterMeshesPool, RenameComponent);
}

#if WITH_EDITOR
void UText3DStaticMeshesRenderer::SetDebugMode(bool bEnabled)
{
	// Since we are dealing with class FProperty, no need to run this for each instance, do it once
	static bool bDebugModeEnabled = true;

	if (bDebugModeEnabled != bEnabled)
	{
		bDebugModeEnabled = bEnabled;

		FProperty* TextRootProperty = FindFProperty<FProperty>(UText3DStaticMeshesRenderer::StaticClass(), GET_MEMBER_NAME_CHECKED(UText3DStaticMeshesRenderer, TextRoot));
		FArrayProperty* CharacterMeshesProperty = FindFProperty<FArrayProperty>(UText3DStaticMeshesRenderer::StaticClass(), GET_MEMBER_NAME_CHECKED(UText3DStaticMeshesRenderer, CharacterMeshes));

		// Here we toggle the CPF_Edit flag to hide/show property in the details panel component editor tree / outliner
		// @see FComponentEditorUtils::GetPropertyForEditableNativeComponent
		// todo : implement a custom debug view widget for text or add a property editor metadata to control the component visibility in the component editor tree / outliner
		if (bEnabled)
		{
			TextRootProperty->SetPropertyFlags(CPF_Edit);
			CharacterMeshesProperty->SetPropertyFlags(CPF_Edit);
		}
		else
		{
			TextRootProperty->ClearPropertyFlags(CPF_Edit);
			CharacterMeshesProperty->ClearPropertyFlags(CPF_Edit);
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE 