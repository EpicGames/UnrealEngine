// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Renderers/Text3DRendererBase.h"
#include "UObject/ObjectPtr.h"
#include "Text3DStaticMeshesRenderer.generated.h"

class USceneComponent;
class UStaticMeshComponent;
class UText3DComponent;

/**
 * Legacy/default renderer for Text3D
 * Each text character is rendered as a StaticMesh within its own StaticMeshComponent
 * Text3DComponent
 * - Text3DRoot (Root)
 * --- Text3DStaticMeshComponent (Character)
 *
 * Eg: The text "Hello" will be rendered using 5 StaticMeshComponents
 */
UCLASS(DisplayName="StaticMeshesRenderer", ClassGroup="Text3D", HideCategories=(Renderer))
class UText3DStaticMeshesRenderer : public UText3DRendererBase
{
	GENERATED_BODY()

public:
	/** Gets the number of font glyphs that are currently used */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Text3D")
	int32 GetGlyphCount();

	/** Gets the StaticMeshComponent of a glyph */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Text3D")
	UStaticMeshComponent* GetGlyphMeshComponent(int32 Index);

	/** Gets all the glyph meshes */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Components|Text3D")
	const TArray<UStaticMeshComponent*>& GetGlyphMeshComponents();

#if WITH_EDITOR
	/** Converts the text to a single static mesh, this is an heavy operation */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ConvertToStaticMesh();
#endif

protected:
	//~ Begin UText3DRendererBase
	virtual void OnCreate() override;
	virtual void OnUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	virtual void OnClear() override;
	virtual void OnDestroy() override;
	virtual EText3DMeshType GetMeshType() const override;
	virtual FName GetFriendlyName() const override;
	virtual FBox OnCalculateBounds() const override;
	virtual void OnIterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const override;
#if WITH_EDITOR
	virtual void OnDebugModeEnabled() override;
	virtual void OnDebugModeDisabled() override;
#endif
	//~ End UText3DRendererBase

	//~ Begin UObject
	virtual void PostRename(UObject* InOldOuter, const FName InOldName) override;
	//~ End UObject

	void AllocateComponents(int32 InNewCharacterCount);

	/** 
	 * If UText3DProjectSettings::UseExistingCharactersAsTemplates is enabled, finds the first valid existing mesh component to use as template
	 * Otherwise this returns null and new character meshes will be default allocated.
	 */
	UStaticMeshComponent* FindTemplate() const;

#if WITH_EDITOR
	void SetDebugMode(bool bEnabled);
#endif

	/** Holds all character components */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Renderer", meta=(EditCondition="false", EditConditionHides))
	TObjectPtr<USceneComponent> TextRoot;

	/** Each character mesh is held in these components */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Renderer", meta=(EditCondition="false", EditConditionHides))
	TArray<TObjectPtr<UStaticMeshComponent>> CharacterMeshes;

	/** Reusable components to avoid creating and destroying them, recycle components or create new ones when pool is empty */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<TObjectPtr<UStaticMeshComponent>> CharacterMeshesPool;
};