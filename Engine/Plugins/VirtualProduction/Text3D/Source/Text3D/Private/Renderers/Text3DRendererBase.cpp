// Copyright Epic Games, Inc. All Rights Reserved.

#include "Renderers/Text3DRendererBase.h"

#include "Extensions/Text3DCharacterExtensionBase.h"
#include "Extensions/Text3DGeometryExtensionBase.h"
#include "Extensions/Text3DLayoutEffectBase.h"
#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "Extensions/Text3DRenderingExtensionBase.h"
#include "Extensions/Text3DStyleExtensionBase.h"
#include "Extensions/Text3DTokenExtensionBase.h"
#include "GameFramework/Actor.h"
#include "Logs/Text3DLogs.h"
#include "Settings/Text3DProjectSettings.h"
#include "Text3DComponent.h"
#include "Text3DDelegates.h"

#if WITH_EDITOR
void UText3DRendererBase::BindDebugDelegate()
{
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->OnSettingChanged().AddUObject(this, &UText3DRendererBase::OnTextSettingsChanged);
	}
}

void UText3DRendererBase::UnbindDebugDelegate()
{
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->OnSettingChanged().RemoveAll(this);
	}
}

void UText3DRendererBase::OnTextSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent)
{
	if (InEvent.GetMemberPropertyName() == UText3DProjectSettings::GetDebugModePropertyName())
	{
		OnDebugModeChanged();
	}
}

void UText3DRendererBase::OnDebugModeChanged()
{
	if (const UText3DProjectSettings* TextSettings = UText3DProjectSettings::Get())
	{
		if (TextSettings->GetDebugMode())
		{
			OnDebugModeEnabled();
		}
		else
		{
			OnDebugModeDisabled();
		}
	}
}
#endif

UText3DComponent* UText3DRendererBase::GetText3DComponent() const
{
	return GetTypedOuter<UText3DComponent>();
}

void UText3DRendererBase::RefreshBounds()
{
	CachedBounds = OnCalculateBounds();
}

void UText3DRendererBase::Create()
{
	if (bInitialized)
	{
		return;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	if (!IsValid(Text3DComponent))
	{
		return;
	}

	OnCreate();

#if WITH_EDITOR
	BindDebugDelegate();
	OnDebugModeChanged();
#endif

	bInitialized = true;

	const AActor* Owner = Text3DComponent->GetOwner();
	UE_LOGF(LogText3D, Verbose, "%ls : Text3DRenderer %ls Created", !!Owner ? *Owner->GetActorNameOrLabel() : TEXT("Invalid owner"), *GetFriendlyName().ToString())
}

void UText3DRendererBase::Update(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
{
	if (bInitialized)
	{
		const UE::Text3D::FText3DBuildContext BuildContext(InParameters);

		UE::Text3D::BroadcastPreRendererUpdate(BuildContext);
		OnUpdate(InParameters);
		UE::Text3D::BroadcastPostRendererUpdate(BuildContext);
	}
}

void UText3DRendererBase::Clear()
{
	if (!bInitialized)
	{
		return;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	if (!Text3DComponent)
	{
		return;
	}

	OnClear();
	CachedBounds.Reset();

	const AActor* Owner = Text3DComponent->GetOwner();
	UE_LOGF(LogText3D, Verbose, "%ls : Text3DRenderer %ls Cleared", !!Owner ? *Owner->GetActorNameOrLabel() : TEXT("Invalid owner"), *GetFriendlyName().ToString())
}

void UText3DRendererBase::Destroy()
{
	if (!bInitialized)
	{
		return;
	}

	const UText3DComponent* Text3DComponent = GetText3DComponent();
	if (!Text3DComponent)
	{
		return;
	}

	OnDestroy();
	CachedBounds.Reset();

#if WITH_EDITOR
	UnbindDebugDelegate();
#endif

	bInitialized = false;

	const AActor* Owner = Text3DComponent->GetOwner();
	UE_LOGF(LogText3D, Verbose, "%ls : Text3DRenderer %ls Destroyed", !!Owner ? *Owner->GetActorNameOrLabel() : TEXT("Invalid owner"), *GetFriendlyName().ToString())
}

FBox UText3DRendererBase::GetBounds() const
{
	if (!CachedBounds.IsSet())
	{
		return FBox(ForceInitToZero);
	}

	return CachedBounds.GetValue();
}

void UText3DRendererBase::IterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const
{
	OnIterateManagedPrimitives(InFunc);
}
