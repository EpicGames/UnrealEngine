// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/Text3DProjectSettings.h"

#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"
#include "Styles/Text3DStyleSet.h"
#include "Subsystems/Text3DEngineSubsystem.h"
#include "UObject/SoftObjectPath.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

FLazyName const UText3DProjectSettings::FMaterialParameters::Mode(TEXT("Mode"));

FLazyName const UText3DProjectSettings::FMaterialParameters::SolidColor(TEXT("Color"));
FLazyName const UText3DProjectSettings::FMaterialParameters::Opacity(TEXT("GlobalOpacity"));

FLazyName const UText3DProjectSettings::FMaterialParameters::GradientOffset(TEXT("GradientOffset"));
FLazyName const UText3DProjectSettings::FMaterialParameters::GradientColorA(TEXT("GradientColorA"));
FLazyName const UText3DProjectSettings::FMaterialParameters::GradientColorB(TEXT("GradientColorB"));
FLazyName const UText3DProjectSettings::FMaterialParameters::GradientRotation(TEXT("GradientRotation"));
FLazyName const UText3DProjectSettings::FMaterialParameters::GradientSmoothness(TEXT("GradientSmoothness"));

FLazyName const UText3DProjectSettings::FMaterialParameters::MainTexture(TEXT("MainTexture"));
FLazyName const UText3DProjectSettings::FMaterialParameters::TexturedUTiling(TEXT("U_Tiling"));
FLazyName const UText3DProjectSettings::FMaterialParameters::TexturedVTiling(TEXT("V_Tiling"));

FLazyName const UText3DProjectSettings::FMaterialParameters::BoundsOrigin(TEXT("TextBoundsOrigin"));
FLazyName const UText3DProjectSettings::FMaterialParameters::BoundsSize(TEXT("TextBoundsSize"));

FLazyName const UText3DProjectSettings::FMaterialParameters::MaskEnabled(TEXT("MaskEnabled"));
FLazyName const UText3DProjectSettings::FMaterialParameters::MaskRotation(TEXT("MaskRotationDegrees"));
FLazyName const UText3DProjectSettings::FMaterialParameters::MaskOffset(TEXT("MaskOffset"));
FLazyName const UText3DProjectSettings::FMaterialParameters::MaskSmoothness(TEXT("MaskSmoothness"));

const UText3DProjectSettings* UText3DProjectSettings::Get()
{
	return GetDefault<UText3DProjectSettings>();
}

UText3DProjectSettings* UText3DProjectSettings::GetMutable()
{
	return GetMutableDefault<UText3DProjectSettings>();
}

UMaterial* UText3DProjectSettings::GetDefaultMaterial() const
{
	return DefaultMaterial.LoadSynchronous();
}

UText3DStyleSet* UText3DProjectSettings::GetDefaultStyleSet() const
{
	return DefaultStyleSet.LoadSynchronous();
}

UFont* UText3DProjectSettings::GetFallbackFont() const
{
	return FallbackFont.LoadSynchronous();
}

UFontFace* UText3DProjectSettings::GetFallbackFontFace() const
{
	return FallbackFontFace.LoadSynchronous();
}

const FString& UText3DProjectSettings::GetFontDirectory() const
{
	return FontDirectory;
}

UMaterial* UText3DProjectSettings::GetBaseMaterial(const FText3DMaterialKey& InKey) const
{
	if (const TSoftObjectPtr<UMaterial>* BaseMaterial = BaseMaterials.Find(InKey))
	{
		return BaseMaterial->LoadSynchronous();
	}

	return nullptr;
}

void UText3DProjectSettings::SetScheduleFontFaceGlyphCleanup(bool bInEnabled)
{
}

bool UText3DProjectSettings::GetScheduleFontFaceGlyphCleanup() const
{
	return false;
}

void UText3DProjectSettings::SetFontFaceGlyphCleanupPeriod(float InValue)
{
}

float UText3DProjectSettings::GetFontFaceGlyphCleanupPeriod() const
{
	return 0.f;
}

void UText3DProjectSettings::SetFontFaceGlyphCleanupDelay(float InDelay)
{
	FontFaceGlyphCleanupDelay = InDelay;
}

UText3DProjectSettings::UText3DProjectSettings()
	: ForceDisableTextCollision(false)
{
	// Config
	CategoryName = TEXT("Text3D");
	SectionName = TEXT("Text3D");

	// Materials
	DefaultMaterial = TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Engine/BasicShapes/BasicShapeMaterial.BasicShapeMaterial")));

	// Fonts
	FallbackFont = TSoftObjectPtr<UFont>(FSoftObjectPath(TEXT("/Engine/EngineFonts/Roboto.Roboto")));
	FallbackFontFace = TSoftObjectPtr<UFontFace>(FSoftObjectPath(TEXT("/Engine/EngineFonts/Faces/DroidSansFallback.DroidSansFallback")));
	FontDirectory = TEXT("/Game/SystemFonts/");

	// Materials
	constexpr bool bIsMaterialUnlit = true;

	// Opaque & Lit
	AddMaterial(FText3DMaterialKey(EText3DMaterialBlendMode::Opaque, !bIsMaterialUnlit)
        , TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Text3D/Materials/M_Text3DOpaqueLit.M_Text3DOpaqueLit"))));

	// Opaque & Unlit
	AddMaterial(FText3DMaterialKey(EText3DMaterialBlendMode::Opaque, bIsMaterialUnlit)
        , TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Text3D/Materials/M_Text3DOpaqueUnlit.M_Text3DOpaqueUnlit"))));

	// Translucent & Lit
	AddMaterial(FText3DMaterialKey(EText3DMaterialBlendMode::Translucent, !bIsMaterialUnlit)
        , TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Text3D/Materials/M_Text3DTranslucentLit.M_Text3DTranslucentLit"))));

	// Translucent & Unlit
	AddMaterial(FText3DMaterialKey(EText3DMaterialBlendMode::Translucent, bIsMaterialUnlit)
	    , TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Text3D/Materials/M_Text3DTranslucentUnlit.M_Text3DTranslucentUnlit"))));
}

void UText3DProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();
	ForceDisableTextCollision.store(bForceDisableTextCollision, std::memory_order_relaxed);
}

void UText3DProjectSettings::AddMaterial(FText3DMaterialKey&& InKey, TSoftObjectPtr<UMaterial>&& InMaterial)
{
	BaseMaterials.Emplace(InKey, InMaterial);
}

#if WITH_EDITOR
void UText3DProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InChangeEvent)
{
	Super::PostEditChangeProperty(InChangeEvent);

	static const TSet<FName> PropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, DefaultMaterial),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, FallbackFont),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, FallbackFontFace),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, FontDirectory),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, FavoriteFonts),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bShowOnlyMonospaced),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bShowOnlyBold),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bShowOnlyItalic),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bDebugMode),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, FontFaceGlyphCleanupDelay),
		GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bUseExistingCharactersAsTemplates),
	};

	const FName MemberName = InChangeEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bForceDisableTextCollision))
	{
		ForceDisableTextCollision.store(bForceDisableTextCollision, std::memory_order_relaxed);
		Save();
	}
	else if (PropertyNames.Contains(MemberName))
	{
		Save();
	}
}

TConstArrayView<FString> UText3DProjectSettings::GetFavoriteFonts() const
{
	return FavoriteFonts;
}

bool UText3DProjectSettings::GetShowOnlyMonospaced() const
{
	return bShowOnlyMonospaced;
}

bool UText3DProjectSettings::GetShowOnlyBold() const
{
	return bShowOnlyBold;
}

bool UText3DProjectSettings::GetShowOnlyItalic() const
{
	return bShowOnlyItalic;
}

void UText3DProjectSettings::AddFavoriteFont(const FString& InFontName)
{
	if (FavoriteFonts.AddUnique(InFontName) == FavoriteFonts.Num() - 1)
	{
		Save();
	}
}

void UText3DProjectSettings::RemoveFavoriteFont(const FString& InFontName)
{
	if (FavoriteFonts.Remove(InFontName) > 0)
	{
		Save();
	}
}

void UText3DProjectSettings::SetShowOnlyMonospaced(bool bInShowOnlyMonospaced)
{
	if (bShowOnlyMonospaced != bInShowOnlyMonospaced)
	{
		bShowOnlyMonospaced = bInShowOnlyMonospaced;
		Save();
	}
}

void UText3DProjectSettings::SetShowOnlyBold(bool bInShowOnlyBold)
{
	if (bShowOnlyBold != bInShowOnlyBold)
	{
		bShowOnlyBold = bInShowOnlyBold;
		Save();
	}
}

void UText3DProjectSettings::SetShowOnlyItalic(bool bInShowOnlyItalic)
{
	if (bShowOnlyItalic != bInShowOnlyItalic)
	{
		bShowOnlyItalic = bInShowOnlyItalic;
		Save();
	}
}

FName UText3DProjectSettings::GetDebugModePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, bDebugMode);
}

FName UText3DProjectSettings::GetDefaultMaterialPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UText3DProjectSettings, DefaultMaterial);
}

bool UText3DProjectSettings::GetDebugMode() const
{
	return bDebugMode;
}
#endif

bool UText3DProjectSettings::ShouldForceDisableTextCollision() const
{
	return ForceDisableTextCollision.load(std::memory_order_relaxed);
}

#if WITH_EDITOR
void UText3DProjectSettings::OpenEditorSettingsWindow() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->ShowViewer(GetContainerName(), GetCategoryName(), GetSectionName());
	}
}

void UText3DProjectSettings::Save()
{
	SaveConfig();

	FPropertyChangedEvent Event(nullptr);
	SettingsChangedDelegate.Broadcast(this, Event);
}
#endif
