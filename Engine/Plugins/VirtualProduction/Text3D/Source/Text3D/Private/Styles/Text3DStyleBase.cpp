// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/Text3DStyleBase.h"

#include "Engine/Font.h"
#include "Settings/Text3DProjectSettings.h"
#include "Styles/Text3DStyleSet.h"
#include "Styling/StyleDefaults.h"
#include "Text3DComponent.h"

#if WITH_EDITOR
void UText3DStyleBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberProperty = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, Font))
	{
		OnFontChanged();
	}
	else if (MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, StyleName)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, bOverrideFont)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, FontTypeface)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, bOverrideFontSize)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, FontSize)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, bOverrideFrontColor)
		|| MemberProperty == GET_MEMBER_NAME_CHECKED(UText3DStyleBase, FrontColor))
	{
		OnStylePropertiesChanged();
	}
}
#endif

UText3DStyleBase::UText3DStyleBase()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		Font = Text3DSettings->GetFallbackFont();
		OnFontChanged();
	}
}

void UText3DStyleBase::SetStyleName(FName InName)
{
	if (InName == StyleName)
	{
		return;
	}

	StyleName = InName;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetOverrideFont(bool bInOverride)
{
	if (bOverrideFont == bInOverride)
	{
		return;
	}

	bOverrideFont = bInOverride;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetFont(UFont* InFont)
{
	if (InFont == Font)
	{
		return;
	}

	Font = InFont;
	OnFontChanged();
}

void UText3DStyleBase::SetFontTypeface(FName InTypeface)
{
	if (InTypeface == FontTypeface)
	{
		return;
	}

	if (!GetTypefaceNames().Contains(InTypeface))
	{
		return;
	}

	FontTypeface = InTypeface;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetOverrideFontSize(bool bInOverride)
{
	if (bOverrideFontSize == bInOverride)
	{
		return;
	}

	bOverrideFontSize = bInOverride;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetFontSize(float InSize)
{
	InSize = FMath::Max(InSize, 0.1f);
	if (FMath::IsNearlyEqual(FontSize, InSize))
	{
		return;
	}

	FontSize = InSize;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetOverrideFrontColor(bool bInOverride)
{
	if (bOverrideFrontColor == bInOverride)
	{
		return;
	}

	bOverrideFrontColor = bInOverride;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::SetFrontColor(FLinearColor InColor)
{
	if (FrontColor.Equals(InColor))
	{
		return;
	}

	FrontColor = InColor;
	OnStylePropertiesChanged();
}

void UText3DStyleBase::ConfigureStyle(FTextBlockStyle& OutStyle) const
{
	if (bOverrideFont)
	{
		FSlateFontInfo FontInfo(Font, OutStyle.Font.Size);
		FontInfo.CompositeFont = Font && Font->GetCompositeFont() ? MakeShared<FCompositeFont>(*Font->GetCompositeFont()) : FStyleDefaults::GetFontInfo().CompositeFont;
		FontInfo.TypefaceFontName = FontTypeface;
		FontInfo.FontFallback = EFontFallback::FF_NoFallback;
		OutStyle.SetFont(FontInfo);
	}

	if (bOverrideFontSize)
	{
		OutStyle.SetFontSize(FontSize);
	}

	if (bOverrideFrontColor)
	{
		OutStyle.SetColorAndOpacity(FrontColor);
	}
}

void UText3DStyleBase::OnStylePropertiesChanged()
{
	if (!StyleName.IsNone())
	{
		if (UText3DComponent* Text3DComponent = GetTypedOuter<UText3DComponent>())
		{
			Text3DComponent->RequestUpdate(EText3DRendererFlags::All);
		}
		else if (UText3DStyleSet* StyleSet = GetTypedOuter<UText3DStyleSet>())
		{
			StyleSet->OnStyleSetPropertiesChanged();
		}
	}
}

void UText3DStyleBase::OnFontChanged()
{
	const TArray<FName> Typefaces = GetTypefaceNames();
	if (!Typefaces.Contains(FontTypeface) && !Typefaces.IsEmpty())
	{
		FontTypeface = Typefaces[0];
	}
	OnStylePropertiesChanged();
}

TArray<FName> UText3DStyleBase::GetTypefaceNames() const
{
	TArray<FName> TypefaceNames;

	if (Font)
	{
		for (const FTypefaceEntry& TypeFaceFont : GetAvailableTypefaces())
		{
			TypefaceNames.Add(TypeFaceFont.Name);
		}
	}

	return TypefaceNames;
}

TArrayView<const FTypefaceEntry> UText3DStyleBase::GetAvailableTypefaces() const
{
	if (Font)
	{
		if (const FCompositeFont* CompositeFont = Font->GetCompositeFont())
		{
			return CompositeFont->DefaultTypeface.Fonts;
		}
	}

	return {};
}
