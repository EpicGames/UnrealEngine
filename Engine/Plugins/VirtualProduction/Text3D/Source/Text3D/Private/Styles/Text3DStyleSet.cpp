// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/Text3DStyleSet.h"

UText3DStyleSet::FOnStyleSetUpdated UText3DStyleSet::OnStyleSetUpdatedDelegate;

#if WITH_EDITOR
void UText3DStyleSet::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UText3DStyleSet, Styles))
	{
		OnStyleSetPropertiesChanged();
	}
}
#endif

void UText3DStyleSet::OnStyleSetPropertiesChanged()
{
	OnStyleSetUpdatedDelegate.Broadcast(this);
}
