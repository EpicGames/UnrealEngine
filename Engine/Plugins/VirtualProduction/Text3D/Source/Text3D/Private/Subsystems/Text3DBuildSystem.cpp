// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DBuildSystem.h"
#include "Algo/Contains.h"
#include "Algo/ForEach.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Logs/Text3DLogs.h"
#include "Misc/CoreDelegates.h"
#include "Renderers/Text3DRendererBase.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "Text3DComponent.h"
#include "Text3DEngineSubsystem.h"
#include "Text3DSceneViewExtension.h"
#include "Utilities/Text3DScopedLog.h"

namespace UE::Text3D::Private
{
TAutoConsoleVariable<int32> CVarStarveFrameLimit(TEXT("Text3D.Build.StarveFrameLimit")
	, 10
	, TEXT("The maximum number of frames that the build can starve before forcing an incremental build"));

TAutoConsoleVariable<bool> CVarEnableBuildTimeBudget(TEXT("Text3D.Build.EnableBuildTimeBudget")
	, true
	, TEXT("Whether to have a time budget when building Text3D components.\n"));

TAutoConsoleVariable<float> CVarBuildTimeBudget(TEXT("Text3D.Build.BuildTimeBudget")
	, 0.002f
	, TEXT("The time budget allocated for building Text3D components.\n")
	  TEXT("Requires Text3D.Build.EnableBuildTimeBudget to be true."));

TAutoConsoleVariable<float> CVarBuildGlyphBudgetThreshold(TEXT("Text3D.Build.GlyphBuildBudgetThreshold")
	, 0.25f
	, TEXT("If the build budget used so far (as a %, from 0-1) exceeds this, the glyph building will take place in the next incremental build tick.\n")
	  TEXT("Requires Text3D.Build.EnableBuildTimeBudget to be true."));

/** Returns the scene view extension to use for this build system */
TSharedRef<FText3DSceneViewExtension> GetSceneViewExtension()
{
	static TWeakPtr<FText3DSceneViewExtension> SceneViewExtensionWeak;
	TSharedPtr<FText3DSceneViewExtension> SceneViewExtension = SceneViewExtensionWeak.Pin();
	if (!SceneViewExtension.IsValid())
	{
		SceneViewExtension = FSceneViewExtensions::NewExtension<FText3DSceneViewExtension>();
		SceneViewExtensionWeak = SceneViewExtension;
	}
	return SceneViewExtension.ToSharedRef();
}

} // UE::Text3D::Private

UText3DBuildSystem::UText3DBuildSystem()
	: FTickableGameObject(ETickableTickType::Never)
{
}

UText3DBuildSystem* UText3DBuildSystem::Get(TNotNull<const UText3DComponent*> InComponent)
{
	if (const UWorld* const World = InComponent->GetWorld())
	{
		return World->GetSubsystem<UText3DBuildSystem>();
	}
	return nullptr;
}

void UText3DBuildSystem::RequestTextBuild(TNotNull<UText3DComponent*> InComponent, EText3DRendererFlags InFlags, bool bInImmediateUpdate)
{
	if (InFlags == EText3DRendererFlags::None)
	{
		return;
	}

	// Determine whether to use blocking build
	// For immediate builds, always use blocking builds.
	// For non-immediate blocking builds, determine this by what the component says. 
	const bool bUseBlockingBuild = bInImmediateUpdate || InComponent->GetUseBlockingBuild();

	UE::Text3D::FScopedLog ScopedLog;
	UE_LOGF(LogText3D, VeryVerbose, "%lsRequesting %ls text build for component '%ls'. Flags %0.2d"
		, ScopedLog.GetLogPrefix()
		, bUseBlockingBuild ? TEXT("BLOCKING") : TEXT("incremental")
		, *InComponent->GetReadableName()
		, static_cast<int32>(InFlags));

	// A builder is relevant if it matches the component and has not complete
	auto IsBuilderRelevant = 
		[InComponent](const UE::Text3D::FTextBuilder& InBuilder)->bool
		{
			return InBuilder.GetComponent() == InComponent && !InBuilder.IsComplete();
		};

	// Always allow if build is not immediate. If immediate, ensure the flags are exactly equal. If there's another flag, it can't be used.
	auto AreBuilderFlagsCompatible = 
		[InFlags, bInImmediateUpdate](const UE::Text3D::FTextBuilder& InBuilder)->bool
		{
			return !bInImmediateUpdate || InBuilder.GetFlags() == InFlags;
		};

	// Cancel builders in progress if flags overlap. There could still be builders that haven't yet started where we could queue in our request.
	UE::Text3D::FTextBuilder* Builder = nullptr;
	for (UE::Text3D::FTextBuilder& CandidateBuilder : TextBuilders)
	{
		if (IsBuilderRelevant(CandidateBuilder))
		{
			// Cancel builder if already in progress. A newer update is available.
			// Only applicable if the new flags contain all the builders flags
			// E.g. we don't want to cancel a Geometry&Material build when requesting a Material only build
			// But we do want to cancel a Layout build when requesting a Geometry&Layout build
			if (EnumHasAllFlags(InFlags, CandidateBuilder.GetFlags()))
			{
				UE_LOGF(LogText3D, VeryVerbose, "%lsCanceling active builder '%ls'. A newer build request was made."
					, UE::Text3D::FScopedLog().GetLogPrefix()
					, *CandidateBuilder.GetDebugName());
				CandidateBuilder.CancelBuild();
				continue;
			}

			if (AreBuilderFlagsCompatible(CandidateBuilder) && EnumHasAllFlags(CandidateBuilder.GetFlags(), InFlags))
			{
				UE_LOGF(LogText3D, VeryVerbose, "%lsFound candidate active builder '%ls'."
					, UE::Text3D::FScopedLog().GetLogPrefix()	
					, *CandidateBuilder.GetDebugName());
				Builder = &CandidateBuilder;
				break;
			}
		}
	}

	// Check in the queued builders.
	if (!Builder)
	{
		for (UE::Text3D::FTextBuilder& CandidateBuilder : QueuedBuilders)
		{
			if (IsBuilderRelevant(CandidateBuilder) && AreBuilderFlagsCompatible(CandidateBuilder))
			{
				UE_LOGF(LogText3D, VeryVerbose, "%lsFound candidate queued builder '%ls'."
					, UE::Text3D::FScopedLog().GetLogPrefix()
					, *CandidateBuilder.GetDebugName());
				Builder = &CandidateBuilder;
				break;
			}
		}
	}

	if (!Builder)
	{
		Builder = &QueuedBuilders.Emplace_GetRef(InComponent);
		UE_LOGF(LogText3D, VeryVerbose, "%lsNo candidate text builder was found. Queueing a new builder '%ls'"
			, UE::Text3D::FScopedLog().GetLogPrefix()
			, *Builder->GetDebugName());
	}

	Builder->PrepareFlags(InFlags);
	Builder->SetUseBlockingBuild(bUseBlockingBuild);

	ConditionallyRegisterDelegates();
	SetTickableTickType(ETickableTickType::Always);

	if (bInImmediateUpdate)
	{
		const uint64 StartCycles = FPlatformTime::Cycles64();
		Builder->BlockingBuild();
		const uint64 EndCycles = FPlatformTime::Cycles64();

		// Accumulate the deficit so as not to impact next tick due to multiple blocking builds
		BudgetDeficit += (EndCycles - StartCycles);
	}
}

int32 UText3DBuildSystem::RemoveTextBuild(TNotNull<UText3DComponent*> InComponent)
{
	auto IsMatchingComponent = 
		[InComponent](const UE::Text3D::FTextBuilder& InTextBuilder)
		{
			return InTextBuilder.GetComponent() == InComponent;
		};

	int32 RemovedCount = 0;
	RemovedCount += QueuedBuilders.RemoveAll(IsMatchingComponent);
	RemovedCount += TextBuilders.RemoveAll(IsMatchingComponent);

	UE_CLOGF(RemovedCount > 0, LogText3D, Verbose, "Removed '%ls' from %d builders", *InComponent->GetReadableName(), RemovedCount);
	return RemovedCount;
}

void UText3DBuildSystem::IncrementalTextBuild()
{
	using namespace UE::Text3D;

	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DWorldSubsystem::TextIncrementalUpdate);

	// This was called within the same frame of the request. Skip to next frame to gather the frame duration
	if (!BeginFrameCycles.IsSet())
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		// Invalidate the gotten frame cycles
		BeginFrameCycles.Reset();
	};

	UE::Text3D::FScopedLog ScopedLog;
	UE_LOGF(LogText3D, VeryVerbose, "%lsPerforming incremental Text3D build...", ScopedLog.GetLogPrefix());

	TextBuilders.Append(MoveTemp(QueuedBuilders));
	QueuedBuilders.Reset();

	// Remove all finished builders
	TextBuilders.RemoveAll(
		[](const UE::Text3D::FTextBuilder& InBuilder)
		{
			UE_CLOGF(InBuilder.IsComplete(), LogText3D, VeryVerbose, "%lsRemoving finished builder '%ls'"
				, UE::Text3D::FScopedLog().GetLogPrefix()
				, *InBuilder.GetDebugName());
			return InBuilder.IsComplete();
		});

	if (!IsBuildingText())
	{
		UE_LOGF(LogText3D, VeryVerbose, "%lsIncremental Text3D build finished. No more build jobs left.", UE::Text3D::FScopedLog().GetLogPrefix());
		TextBuilders.Empty();
		QueuedBuilders.Empty();
		SetTickableTickType(ETickableTickType::Never);
		return; // nothing to update
	}

	UText3DEngineSubsystem* const TextSubsystem = UText3DEngineSubsystem::Get();
	if (!ensure(TextSubsystem))
	{
		return;
	}

	const uint64 CurrentCycles = FPlatformTime::Cycles64();

	UE::Text3D::FTextBuilder::FUpdateParams Params;
	if (UE::Text3D::Private::CVarEnableBuildTimeBudget.GetValueOnGameThread())
	{
		Params.BudgetEnd = CurrentCycles + FPlatformTime::SecondsToCycles64(UE::Text3D::Private::CVarBuildTimeBudget.GetValueOnGameThread());
		if (*Params.BudgetEnd <= BudgetDeficit)
		{
			*Params.BudgetEnd = CurrentCycles;
		}
		else
		{
			*Params.BudgetEnd -= BudgetDeficit;
		}
	}
	BudgetDeficit = 0;

	Params.MaxStarvedFrames = Private::CVarStarveFrameLimit.GetValueOnGameThread();
	Params.TempMaxUpdatePhase = EBuildPhase::PrepareExtensions;

	const int32 GlyphMeshBuildCount = TextSubsystem->GetQueuedGlyphMeshBuildCount();

	// Update all the builders that are preparing
	// Queued blocking text builders are also handled here.
	for (UE::Text3D::FTextBuilder& Builder : TextBuilders)
	{
		Builder.Update(Params);
	}

	Params.TempMaxUpdatePhase.Reset();

	// If there are any new glyph mesh builds queued, make sure there's enough budget time to build these
	if (GlyphMeshBuildCount != TextSubsystem->GetQueuedGlyphMeshBuildCount() && Params.BudgetEnd.IsSet())
	{
		// Calculate the amount of budget we've used so far (%)
		const double BuildBudgetPct = FMath::SafeDivide(static_cast<double>(FPlatformTime::Cycles64() - CurrentCycles), static_cast<double>(*Params.BudgetEnd - CurrentCycles));
		if (BuildBudgetPct >= Private::CVarBuildGlyphBudgetThreshold.GetValueOnGameThread())
		{
			return;
		}
	}

	TextSubsystem->ProcessBuildGlyphMeshes();

	// Update all the builders with no phase limit
	for (UE::Text3D::FTextBuilder& Builder : TextBuilders)
	{
		Builder.Update(Params);
	}
}

void UText3DBuildSystem::BlockBuildTexts()
{
	TextBuilders.Append(MoveTemp(QueuedBuilders));
	QueuedBuilders.Empty();

	Algo::ForEach(TextBuilders, &UE::Text3D::FTextBuilder::BlockingBuild);
	TextBuilders.Empty();
}

bool UText3DBuildSystem::IsBuildingText() const
{
	if (!QueuedBuilders.IsEmpty() || !TextBuilders.IsEmpty())
	{
		return true;
	}

	// Assume that if there are levels that are still not fully registered, that text is still pending build.
	// There could be Text3D components that are yet to register to request builds.
	const UWorld* const World = GetWorld();
	for (const ULevel* Level : World->GetLevels())
	{
		if (Level && !Level->bAreComponentsCurrentlyRegistered)
		{
			return true;
		}
	}
	return false;
}

void UText3DBuildSystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);

	SceneViewExtension = UE::Text3D::Private::GetSceneViewExtension();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddWeakLambda(this, 
		[this](const TMap<UObject*, UObject*>& InReplacementMap)
		{
			auto ReplaceObjects = [&InReplacementMap](UE::Text3D::FTextBuilder& InBuilder)
				{
					InBuilder.OnObjectsReplaced(InReplacementMap);
				};
			Algo::ForEach(QueuedBuilders, ReplaceObjects);
			Algo::ForEach(TextBuilders, ReplaceObjects);
		});
#endif
}

void UText3DBuildSystem::Deinitialize()
{
	Super::Deinitialize();
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif
	UnregisterDelegates();
	SceneViewExtension.Reset();
}

bool UText3DBuildSystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType != EWorldType::None;
}

bool UText3DBuildSystem::IsTickableInEditor() const
{
	return true;
}

void UText3DBuildSystem::Tick(float InDeltaTime)
{
	IncrementalTextBuild();
}

TStatId UText3DBuildSystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UText3DBuildSystem, STATGROUP_Tickables);
}

bool UText3DBuildSystem::IsBuildInProgress() const
{
	return IsBuildingText();
}

void UText3DBuildSystem::FlushBuilds()
{
	BlockBuildTexts();
}

void UText3DBuildSystem::HidePrimitivesBeingBuilt(FSceneView& InSceneView) const
{
	auto AddPendingBuildItem = 
		[&InSceneView](const UE::Text3D::FTextBuilder& InBuilder)
		{
			// Only consider builders that are building geometry and during or past the renderer update stage.
			const bool bBuildingGeometryInProgress = EnumHasAnyFlags(InBuilder.GetFlags(), EText3DRendererFlags::Geometry)
				&& InBuilder.GetPhase() >= UE::Text3D::EBuildPhase::RendererUpdate;

			if (!bBuildingGeometryInProgress)
			{
				return;
			}
			if (const UText3DComponent* const Component = InBuilder.GetComponent())
			{
				if (const UText3DRendererBase* const TextRenderer = Component->GetTextRenderer())
				{
					TextRenderer->IterateManagedPrimitives(
						[&InSceneView](TNotNull<const UPrimitiveComponent*> InComponent)
						{
							InSceneView.HiddenPrimitives.Add(InComponent->GetPrimitiveSceneId());
						});
				}
			}
		};
	Algo::ForEach(QueuedBuilders, AddPendingBuildItem);
	Algo::ForEach(TextBuilders, AddPendingBuildItem);
}

void UText3DBuildSystem::ConditionallyRegisterDelegates()
{
	if (!BeginFrameHandle.IsValid())
	{
		BeginFrameHandle = FCoreDelegates::OnBeginFrame.AddWeakLambda(this, [this]
			{
				BeginFrameCycles = FPlatformTime::Cycles64();
			});
	}
}

void UText3DBuildSystem::UnregisterDelegates()
{
	FCoreDelegates::OnBeginFrame.Remove(BeginFrameHandle);
	BeginFrameHandle.Reset();
}
