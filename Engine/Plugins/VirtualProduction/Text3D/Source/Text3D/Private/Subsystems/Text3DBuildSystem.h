// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "IText3DBuildSystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "Text3DBuilder.h"
#include "Text3DBuildSystem.generated.h"

namespace UE::Text3D
{
	class FText3DSceneViewExtension;
}

class UText3DComponent;
enum class EText3DRendererFlags : uint8;

/**
 * Manages all the text builds for a world 
 * It's different from UText3DEngineSubsystem to keep these text builders referencing the text components scoped within the same world.
 */
UCLASS(BlueprintType)
class UText3DBuildSystem : public UWorldSubsystem, public FTickableGameObject, public IText3DBuildSystemInterface
{
	GENERATED_BODY()

public:
	UText3DBuildSystem();

	static UText3DBuildSystem* Get(TNotNull<const UText3DComponent*> InComponent);

	/** 
	 * Adds the text component to the queue of builders 
	 * @param InComponent the component to request build for
	 * @param InFlags the renderer flags for the build
	 * @param bInBlockingBuild whether the build should be blocking (immediate) or incremental (deferred, across multiple frames)
	 */
	void RequestTextBuild(TNotNull<UText3DComponent*> InComponent, EText3DRendererFlags InFlags, bool bInBlockingBuild);

	/** Removes the builders that relate to the given component*/
	int32 RemoveTextBuild(TNotNull<UText3DComponent*> InComponent);

	/** Called on tick to process builders and call IncrementalBuild on them */
	void IncrementalTextBuild();

	/**
	 * Executes any remaining text builds with no time budget. 
	 * Warning: This can cause a hitch and should only be used when the result is needed immediately regardless of hitch.
	 */
	UFUNCTION(BlueprintCallable, Category="Text 3D")
	void BlockBuildTexts();

	/** Returns true if there are Text3D components being built */
	UFUNCTION(BlueprintPure, Category="Text 3D")
	bool IsBuildingText() const;

protected:
	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

	//~ Begin FTickableGameObject
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject

	//~ Begin IText3DBuildSystemInterface
	virtual bool IsBuildInProgress() const override;
	virtual void FlushBuilds() override;
	virtual void HidePrimitivesBeingBuilt(FSceneView& InSceneView) const override;
	//~ End IText3DBuildSystemInterface

private:
	/** Registers the build delegates only if they haven't been registered yet */
	void ConditionallyRegisterDelegates();

	/** Unregisters the build delegates */
	void UnregisterDelegates();

	/** Builders that are not yet processed */
	TArray<UE::Text3D::FTextBuilder> QueuedBuilders;

	/** Active builders */
	TArray<UE::Text3D::FTextBuilder> TextBuilders;

	/** Scene view extension to hide any primitive from text actors that are pending build */
	TSharedPtr<UE::Text3D::FText3DSceneViewExtension> SceneViewExtension;

	/** Cycles measured in begin frame */
	TOptional<uint64> BeginFrameCycles;

	/** Accumulated cycles spent on blocking updates to determine if next tick an update should happen */
	uint64 BudgetDeficit = 0;

	/** Handle to the begin frame call to keep track how long this frame's been going for */
	FDelegateHandle BeginFrameHandle;
};
