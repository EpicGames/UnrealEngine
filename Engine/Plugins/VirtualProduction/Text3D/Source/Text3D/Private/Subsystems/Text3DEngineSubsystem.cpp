// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DEngineSubsystem.h"

#include "Async/ParallelFor.h"
#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "GeometryBuilders/Text3DGlyphContourNode.h"
#include "GeometryBuilders/Text3DGlyphLoader.h"
#include "GeometryBuilders/Text3DGlyphMeshBuilder.h"
#include "GeometryBuilders/Text3DGlyphOutline.h"
#include "Logs/Text3DLogs.h"
#include "Materials/Material.h"
#include "Misc/FileHelper.h"
#include "Settings/Text3DProjectSettings.h"
#include "Text3DInternalTypes.h"
#include "UDynamicMesh.h"
#include "UObject/ConstructorHelpers.h"

#if USING_INSTRUMENTATION
#include "Sanitizer/RaceDetector.h"
#endif

#if WITH_FREETYPE
#include "Fonts/FontCacheFreeType.h"
#endif

namespace UE::Text3D::Private
{

#if USING_INSTRUMENTATION
static bool GDetectRaceDuringBuild = false;
static FAutoConsoleVariableRef CVarDetectRaceDuringBuild(
	TEXT("Text3D.Build.DetectRace"),
	GDetectRaceDuringBuild,
	TEXT("Activate the race detector when building text in parallel"),
	ECVF_Default
);
#endif

} // UE::Text3D::Private

UText3DEngineSubsystem* UText3DEngineSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UText3DEngineSubsystem>();
	}

	return nullptr;
}

void UText3DEngineSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);

	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
#if WITH_EDITOR
		Text3DSettings->OnSettingChanged().AddUObject(this, &UText3DEngineSubsystem::OnText3DSettingsChanged);
#endif
		UpdateDefaultMaterial();
	}
}

void UText3DEngineSubsystem::Deinitialize()
{
	Super::Deinitialize();
	ClearCache();

#if WITH_EDITOR
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->OnSettingChanged().RemoveAll(this);
	}
#endif
}

bool UText3DEngineSubsystem::Exec_Dev(UWorld* InWorld, const TCHAR* InCmd, FOutputDevice& InAr)
{
	if (HasAllFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	if (FParse::Command(&InCmd, TEXT("text3d")))
	{
		if (FParse::Command(&InCmd, TEXT("cache")))
		{
			if (FParse::Command(&InCmd, TEXT("show")))
			{
				PrintCache();
				return true;
			}

			if (FParse::Command(&InCmd, TEXT("cleanup")))
			{
				if (bGlyphMeshBuildInProgress)
				{
					UE_LOGF(LogText3D, Warning, "Cleanup failed. Text3D Engine Subsystem is currently building glyph meshes.");
				}
				else
				{
					CleanupCache(0);	
				}
				return true;
			}

			if (FParse::Command(&InCmd, TEXT("clear")))
			{
				if (bGlyphMeshBuildInProgress)
				{
					UE_LOGF(LogText3D, Warning, "Clear cache failed. Text3D Engine Subsystem is currently building glyph meshes.");
				}
				else
				{
					ClearCache();	
				}
				return true;
			}
		}
	}

	return false;
}

#if WITH_EDITOR
void UText3DEngineSubsystem::OnText3DSettingsChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetMemberPropertyName() == UText3DProjectSettings::GetDefaultMaterialPropertyName())
	{
		UpdateDefaultMaterial();
	}
}
#endif

void UText3DEngineSubsystem::UpdateDefaultMaterial()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		CachedDefaultMaterial = Text3DSettings->GetDefaultMaterial();
	}
}

void UText3DEngineSubsystem::ScheduleCacheCleanup()
{
	// Already queued up. Nothing else to do.
	if (CacheCleanupDelegate.IsValid())
	{
		return;
	}

	const UText3DProjectSettings* const TextSettings = UText3DProjectSettings::Get();
	const float CleanupDelay = TextSettings ? TextSettings->GetFontFaceGlyphCleanupDelay() : 5.f;

	UE_LOGF(LogText3D, Verbose, "Text3D Engine Subsystem cache cleanup requested.");

	// Immediate cleanup if close to zero or negative.
	if (CleanupDelay <= UE_SMALL_NUMBER && !bGlyphMeshBuildInProgress)
	{
		CleanupCache(0.f);
	}
	else
	{
		CacheCleanupDelegate = FTSTicker::GetCoreTicker()
			.AddTicker(FTickerDelegate::CreateUObject(this, &UText3DEngineSubsystem::CleanupCache), CleanupDelay);
	}
}

UMaterial* UText3DEngineSubsystem::GetDefaultMaterial() const
{
	return CachedDefaultMaterial;
}

bool UText3DEngineSubsystem::CleanupCache(float)
{
	if (bGlyphMeshBuildInProgress)
	{
		// Wait until next tick to see if cache can be cleaned up, glyph mesh building in progress
		UE_LOGF(LogText3D, Verbose, "Text3D Engine Subsystem cache cleanup could not be performed while glyph mesh building in progress. Trying again next call.");
		return true;
	}

	UE_LOGF(LogText3D, Verbose, "Text3D Engine Subsystem performing cache cleanup...");

	// Removing CleanupHandle Ticker while still executing this ticker delegate is by-design valid.
	// see TickerTests.cpp:112 "a delegate removal from inside the delegate execution (used to be a deadlock)"
	FTSTicker::RemoveTicker(CacheCleanupDelegate);
	CacheCleanupDelegate.Reset();

	uint32 GlyphMeshCount = 0;
	uint32 GlyphMeshRemoved = 0;
	uint32 FontFaceRemoved = 0;

	for (TMap<uint32, FText3DFontFaceCache>::TIterator It(CachedFontFaces); It; ++It)
	{
		FText3DFontFaceCache& FontFace = It.Value();
		GlyphMeshRemoved += FontFace.CleanupCache();
		GlyphMeshCount += FontFace.GetCacheGlyphCount();
		if (!FontFace.IsValid() || FontFace.GetCacheGlyphCount() == 0)
		{
			It.RemoveCurrent();
			FontFaceRemoved++;
		}
	}

	if (FontFaceRemoved > 0 || GlyphMeshRemoved > 0)
	{
		UE_LOGF(LogText3D, Verbose, "Text3D Engine Subsystem cache cleanup completed : %i unused glyph(s) removed, %i remaining glyph(s), %i unused font face(s) removed, %i remaining font face(s)", GlyphMeshRemoved, GlyphMeshCount, FontFaceRemoved, CachedFontFaces.Num());
	}

	// one time off.
	return false;
}

void UText3DEngineSubsystem::PrintCache() const
{
	UE_LOGF(LogText3D, Log, "Text3D Engine Subsystem is currently caching %i fonts: ", CachedFontFaces.Num());
	for (const TPair<uint32, FText3DFontFaceCache>& CachedFont : CachedFontFaces)
	{
		UE_LOGF(LogText3D, Log, "\t%ls", *CachedFont.Value.ToDebugString());
	}
}

void UText3DEngineSubsystem::ClearCache()
{
	CachedFontFaces.Empty();
}

FText3DFontFaceCache* UText3DEngineSubsystem::FindCachedFontFace(uint32 InFontFaceHash)
{
	return CachedFontFaces.Find(InFontFaceHash);
}

void UText3DEngineSubsystem::QueueBuildGlyphMeshes(TArray<FText3DBuildGlyphMeshDesc>&& InMeshDescs, const FGlyphMeshParameters& InGlyphMeshParameters)
{
	if (InMeshDescs.IsEmpty())
	{
		return;
	}

	const int32 GlyphParametersIndex = QueuedGlyphMeshParameters.Add(InGlyphMeshParameters);
	for (FText3DBuildGlyphMeshDesc& MeshDesc : InMeshDescs)
	{
		MeshDesc.GlyphParametersIndex = GlyphParametersIndex;
	}
	QueuedMeshDescs.Append(MoveTemp(InMeshDescs));
}

int32 UText3DEngineSubsystem::GetQueuedGlyphMeshBuildCount() const
{
	return QueuedMeshDescs.Num();
}

void UText3DEngineSubsystem::ProcessBuildGlyphMeshes()
{
	// If queued mesh desc is empty, the mesh parameters should also be.
	if (QueuedMeshDescs.IsEmpty() && ensure(QueuedGlyphMeshParameters.IsEmpty()))
	{
		return;
	}

	TGuardValue<bool> BuildScope(bGlyphMeshBuildInProgress, true);

	const TArray<FGlyphMeshParameters> GlyphParameters = MoveTemp(QueuedGlyphMeshParameters);
	QueuedGlyphMeshParameters.Reset();

	const TArray<FText3DBuildGlyphMeshDesc> MeshDescs = MoveTemp(QueuedMeshDescs);
	QueuedMeshDescs.Reset();

#if WITH_FREETYPE
	TRACE_CPUPROFILER_EVENT_SCOPE(UText3DEngineSubsystem::BuildGlyphMeshes);
	const FText3DFontFaceCache::FBuildGlyphMeshSharedDesc SharedParams
		{
			.TextSubsystem = this,
			.DefaultMaterial = GetDefaultMaterial(),
		};

	// Each mesh build time cost is highly variable, and each task can take enough time to justify the cost of synchronization
	constexpr EParallelForFlags ParallelForFlags = EParallelForFlags::Unbalanced;

#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::FRaceDetectorScope RaceDetectorScope(UE::Text3D::Private::GDetectRaceDuringBuild);
#endif

	ParallelFor(MeshDescs.Num(), 
		[&SharedParams, &MeshDescs, &GlyphParameters](int32 InMeshDescIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UText3DEngineSubsystem::BuildGlyphMesh);
			const FText3DBuildGlyphMeshDesc& MeshDesc = MeshDescs[InMeshDescIndex];

			if (FText3DFontFaceCache* FontFaceCache = SharedParams.TextSubsystem->FindCachedFontFace(MeshDesc.FontFace))
			{
				const FText3DFontFaceCache::FBuildGlyphMeshInstanceDesc InstanceParams
					{
						.GlyphOutline = &MeshDesc.GlyphOutline,
						.GlyphMeshParameters = &GlyphParameters[MeshDesc.GlyphParametersIndex],
						.GlyphIndex = MeshDesc.GlyphIndex,
					};
				FontFaceCache->BuildGlyphMesh(SharedParams, InstanceParams);
			}
		}, ParallelForFlags);
#endif // WITH_FREETYPE
}

#if WITH_FREETYPE
FText3DFontFaceCache* UText3DEngineSubsystem::FindOrAddCachedFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace)
{
	if (!InFontFace.IsValid() || !InFontFace->IsFaceValid())
	{
		return nullptr;
	}

	const uint32 FontFaceHash = FText3DFontFaceCache::GetFontFaceHash(InFontFace);

	if (FontFaceHash == 0)
	{
		return nullptr;
	}

	FText3DFontFaceCache& FontFaceCache = CachedFontFaces.FindOrAdd(FontFaceHash, FText3DFontFaceCache(InFontFace));

	/** Refresh font face with latest valid version */
	FontFaceCache.UpdateFontFace(InFontFace);

	return &FontFaceCache;
}

FText3DFontFaceCache* UText3DEngineSubsystem::FindCachedFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace)
{
	if (!InFontFace.IsValid() || !InFontFace->IsFaceValid())
	{
		return nullptr;
	}

	const uint32 FontFaceHash = FText3DFontFaceCache::GetFontFaceHash(InFontFace);
	if (FontFaceHash == 0)
	{
		return nullptr;
	}

	return CachedFontFaces.Find(FontFaceHash);
}

uint32 FText3DFontFaceCache::GetFontFaceHash(const TSharedPtr<FFreeTypeFace>& InFontFace)
{
	if (const FT_Face Face = InFontFace->GetFace())
	{
		if (Face->family_name != nullptr && Face->style_name != nullptr)
		{
			const FString FaceFamily(Face->family_name);
			const FString FaceStyle(Face->style_name);
			const int32 FaceIndex = Face->face_index;
			return HashCombine(HashCombine(GetTypeHash(FaceFamily), GetTypeHash(FaceStyle)), FaceIndex);
		}
	}

	return 0;
}

FText3DFontFaceCache::FText3DFontFaceCache(const TSharedPtr<FFreeTypeFace>& InFontFace)
	: FontFace(InFontFace)
	, FontFaceHash(GetFontFaceHash(InFontFace))
{
}

void FText3DFontFaceCache::UpdateFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace)
{
	FontFace = InFontFace;
}
#endif

#if WITH_FREETYPE
TOptional<FText3DBuildGlyphMeshDesc> FText3DFontFaceCache::PrepareCachedMesh(uint32 InGlyphIndex, const FGlyphMeshParameters& InParameters, bool& bOutAlreadyCached)
{
	bOutAlreadyCached = false;

	const uint32 HashParameters = GetGlyphMeshHash(InGlyphIndex, InParameters);
	if (FText3DCachedMesh* CachedMesh = GlyphMeshes.Find(HashParameters))
	{
		bOutAlreadyCached = true;
		return {};
	}

	TOptional<UE::Text3D::FGlyphOutline> GlyphOutline = LoadGlyphOutline(InGlyphIndex);
	if (!GlyphOutline.IsSet())
	{
		return {};
	}

	FText3DCachedMesh& CachedMesh = GlyphMeshes.Add(HashParameters);

	// Font face is already valid if a valid root node was produced
	if (const FT_Face Face = FontFace->GetFace())
	{
		CachedMesh.FontFaceGlyphSize = FVector2D(Face->size->metrics.x_ppem, Face->size->metrics.y_ppem);
	}

	FText3DBuildGlyphMeshDesc MeshDesc;
	MeshDesc.GlyphOutline = MoveTemp(*GlyphOutline);
	MeshDesc.FontFace = FontFace;
	MeshDesc.GlyphIndex = InGlyphIndex;
	return MeshDesc;
}
#endif // WITH_FREETYPE

uint32 FText3DFontFaceCache::GetGlyphMeshHash(uint32 InGlyphIndex, const FGlyphMeshParameters& InParameters)
{
	return HashCombine(GetTypeHash(InParameters), GetTypeHash(InGlyphIndex));
}

#if WITH_FREETYPE
void FText3DFontFaceCache::BuildGlyphMesh(const FBuildGlyphMeshSharedDesc& InSharedParams, const FBuildGlyphMeshInstanceDesc& InInstanceParams)
{
	TOptional<FText3DGlyphContourNode> RootNode;
	{
		FText3DGlyphLoader GlyphLoader;
		RootNode = GlyphLoader.GenerateContourList(*InInstanceParams.GlyphOutline);
	}

	if (!RootNode.IsSet() || RootNode->Children.IsEmpty())
	{
		return;
	}

	const FGlyphMeshParameters& GlyphMeshParameters = *InInstanceParams.GlyphMeshParameters;

	const uint32 HashParameters = GetGlyphMeshHash(InInstanceParams.GlyphIndex, GlyphMeshParameters);
	FText3DCachedMesh& CachedMesh = GlyphMeshes.FindChecked(HashParameters);

	auto ClearAsyncFlags = [bIsInGameThread = IsInGameThread()](UObject* InObject)
		{
			if (!bIsInGameThread && InObject)
			{
				// OK to clear the async flags for the objects immediately because they have already been assigned directly to a UPROPERTY.
				InObject->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);

				// Subobjects created under the object might still be marked as async.
				// E.g. for UStaticMesh these objects get created with async flag:
				//   - BodySetup
				//   - AssetImportData
				//   - For HiResSourceModel:
				//      - FStaticMeshSourceModel::StaticMeshDescriptionBulkData
				//      - FStaticMeshSourceModel::StaticMeshDescriptionBulkData.PreallocatedMeshDescription
				//   - For SourceModels:
				//      - Only FStaticMeshSourceModel::StaticMeshDescriptionBulkData.PreallocatedMeshDescription
				//        because the StaticMeshDescriptionBulkData has its async flag cleared in FStaticMeshSourceModel::CreateSubObjects.
				ForEachObjectWithOuter(InObject, [](UObject* InSubobject)
					{
						InSubobject->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
					}
					, EGetObjectsFlags::IncludeNestedObjects);
			}
		};

	// UObjects like BodySetup get created while building mesh descriptions, so scope will need to cover those too.
	FGCScopeGuard ScopeGuard;
	ON_SCOPE_EXIT
	{
		// Once finished, clear the async flags of the meshes and its subobjects
		ClearAsyncFlags(CachedMesh.StaticMesh);
		ClearAsyncFlags(CachedMesh.DynamicMesh);
	};

	CachedMesh.StaticMesh = NewObject<UStaticMesh>(InSharedParams.TextSubsystem, NAME_None, RF_Transient | RF_Public);
	if (GlyphMeshParameters.MeshType == EText3DMeshType::Dynamic)
	{
		CachedMesh.DynamicMesh = NewObject<UDynamicMesh>(InSharedParams.TextSubsystem, NAME_None, RF_Transient | RF_Public);
	}

	FText3DGlyphMeshBuilder MeshCreator;
	MeshCreator.CreateMeshes(
		*RootNode,
		GlyphMeshParameters.Extrude,
		GlyphMeshParameters.Bevel,
		GlyphMeshParameters.BevelType,
		GlyphMeshParameters.BevelSegments,
		GlyphMeshParameters.bOutline,
		GlyphMeshParameters.OutlineExpand,
		GlyphMeshParameters.OutlineType
	);
	MeshCreator.SetFrontAndBevelTextureCoordinates(GlyphMeshParameters.Bevel);
	MeshCreator.MirrorGroups(GlyphMeshParameters.Extrude);
	MeshCreator.MovePivot(GlyphMeshParameters.PivotOffset);
	MeshCreator.BuildMesh(CachedMesh, GlyphMeshParameters.MeshType, InSharedParams.DefaultMaterial);
}
#endif // WITH_FREETYPE

const FText3DCachedMesh* FText3DFontFaceCache::FindGlyphMesh(uint32 InGlyphHash) const
{
	return GlyphMeshes.Find(InGlyphHash);
}

FText3DCachedMesh* FText3DFontFaceCache::FindGlyphMesh(uint32 InGlyphHash)
{
	return GlyphMeshes.Find(InGlyphHash);
}

uint32 FText3DFontFaceCache::CleanupCache()
{
	uint32 GlyphMeshRemoved = 0;
	for (TMap<uint32, FText3DCachedMesh>::TIterator It(GlyphMeshes); It; ++It)
	{
		if (It->Value.RefCount == 0)
		{
			It.RemoveCurrent();
			GlyphMeshRemoved++;
		}
	}
	return GlyphMeshRemoved;
}

#if WITH_FREETYPE
TOptional<UE::Text3D::FGlyphOutline> FText3DFontFaceCache::LoadGlyphOutline(uint32 InGlyphIndex) const
{
	if (!FontFace)
	{
		return {};
	}

	const FT_Face Face = FontFace->GetFace();
	if (!Face)
	{
		UE_LOGF(LogText3D, Error, "Failed to load font face glyph contours '%u %i' due to invalid face", FontFaceHash, InGlyphIndex);
		return {};
	}

	const FT_Error Result = FT_Load_Glyph(Face, InGlyphIndex, FT_LOAD_DEFAULT);
	if (Result != 0 || !Face->glyph)
	{
		UE_LOGF(LogText3D, Error, "Failed to load font face glyph contours '%u %i' with error code : %i", FontFaceHash, InGlyphIndex, Result);
		return {};
	}

	return UE::Text3D::FGlyphOutline::MakeOutline(Face->glyph->outline);
}
#endif // WITH_FREETYPE

FString FText3DFontFaceCache::ToDebugString() const
{
#if WITH_FREETYPE
	if (!IsValid())
	{
		return TEXT("(invalid font face cache)");
	}

	const FT_Face Face = FontFace->GetFace();
	return FString::Printf(TEXT("Cached Font Face Family=%s Style=%s Index=%li Hash=%u Meshes=%i Usage=%i")
		, Face->family_name ? *FString(Face->family_name) : TEXT("(invalid)")
		, Face->style_name ? *FString(Face->style_name) : TEXT("(invalid)")
		, Face->face_index
		, FontFaceHash
		, GlyphMeshes.Num()
		, GetCacheRefCount());
#else
	return FString();
#endif
}

bool FText3DFontFaceCache::IsValid() const
{
#if WITH_FREETYPE
	return FontFace.IsValid() && FontFace->GetFace() != nullptr;
#else
	return false;
#endif
}

uint32 FText3DFontFaceCache::GetCacheRefCount() const
{
	uint32 MeshCount = 0;
	for (const TPair<uint32, FText3DCachedMesh>& GlyphMeshPair : GlyphMeshes)
	{
		MeshCount += GlyphMeshPair.Value.RefCount;
	}
	return MeshCount;
}

uint32 FText3DFontFaceCache::GetCacheGlyphCount() const
{
	return GlyphMeshes.Num();
}
