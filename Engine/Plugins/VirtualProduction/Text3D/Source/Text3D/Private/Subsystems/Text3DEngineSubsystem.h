// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "GeometryBuilders/Text3DGlyphOutline.h"
#include "Subsystems/EngineSubsystem.h"
#include "Text3DTypes.h"
#include "Text3DEngineSubsystem.generated.h"

class FFreeTypeFace;
class UMaterial;
class UText3DEngineSubsystem;
struct FSharedStruct;

/** Struct containing the necessary information to build glyphs */
struct FText3DBuildGlyphMeshDesc
{
#if WITH_FREETYPE
	/** Generated outline to build from */
	UE::Text3D::FGlyphOutline GlyphOutline;
	/** Shared font face */
	TSharedPtr<FFreeTypeFace> FontFace;
#endif // WITH_FREETYPE
	/** Index to the glyph to build */
	uint32 GlyphIndex = 0;
	/** Set when queuing the mesh desc to map to the glyph parameters used */
	int32 GlyphParametersIndex = 0;
};

USTRUCT()
struct FText3DFontFaceCache
{
	GENERATED_BODY()

#if WITH_FREETYPE
	static uint32 GetFontFaceHash(const TSharedPtr<FFreeTypeFace>& InFontFace);
#endif
	static uint32 GetGlyphMeshHash(uint32 InGlyphIndex, const FGlyphMeshParameters& InParameters);

	FText3DFontFaceCache() = default;

#if WITH_FREETYPE
	explicit FText3DFontFaceCache(const TSharedPtr<FFreeTypeFace>& InFontFace);
	void UpdateFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace);
#endif

#if WITH_FREETYPE
	/**
	 * Loads the glyph contours and allocates the cached meshes the BuildGlyphMesh will be using (across multiple threads)
	 * This happens in the game thread.
	 * @param InGlyphIndex the glyph index to get the hash for
	 * @param InParameters the mesh parameters for the glyph
	 * @param bOutAlreadyCached is set to true if the glyph mesh is already cached
	 * @return the glyph mesh desc if a new mesh was prepared. Unset otherwise (if already cached, or if glyph outline load failed).
	 */
	TOptional<FText3DBuildGlyphMeshDesc> PrepareCachedMesh(uint32 InGlyphIndex, const FGlyphMeshParameters& InParameters, bool& bOutAlreadyCached);

	/** Data that does not change for a batch build */
	struct FBuildGlyphMeshSharedDesc
	{
		TNotNull<UText3DEngineSubsystem*> TextSubsystem;
		UMaterialInterface* DefaultMaterial;
	};
	/** Data that changes per build call in a batch */
	struct FBuildGlyphMeshInstanceDesc
	{
		TNotNull<const UE::Text3D::FGlyphOutline*> GlyphOutline;
		TNotNull<const FGlyphMeshParameters*> GlyphMeshParameters;
		uint32 GlyphIndex;
	};
	/** 
	 * Builds a glyph mesh and saves the mesh (static and optionally dynamic) to the cached mesh struct 
	 * This is called from multiple threads.
	 */
	void BuildGlyphMesh(const FBuildGlyphMeshSharedDesc& InSharedParams, const FBuildGlyphMeshInstanceDesc& InInstanceParams);
#endif // WITH_FREETYPE

	const FText3DCachedMesh* FindGlyphMesh(uint32 InGlyphHash) const;
	FText3DCachedMesh* FindGlyphMesh(uint32 InGlyphHash);
	uint32 CleanupCache();
	
	FString ToDebugString() const;
	bool IsValid() const;
	uint32 GetCacheRefCount() const;
	uint32 GetCacheGlyphCount() const;

	bool operator==(const FText3DFontFaceCache& InOther) const
	{
		return FontFaceHash == InOther.FontFaceHash;
	}

	bool operator!=(const FText3DFontFaceCache& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FText3DFontFaceCache& InValue)
	{
		return InValue.FontFaceHash;
	}

private:
#if WITH_FREETYPE
	TOptional<UE::Text3D::FGlyphOutline> LoadGlyphOutline(uint32 InGlyphIndex) const;

	/** Font face for a Font+Typeface */
	TSharedPtr<FFreeTypeFace> FontFace;
#endif // WITH_FREETYPE

	/** Glyph indexes to cached meshes */
	UPROPERTY()
	TMap<uint32, FText3DCachedMesh> GlyphMeshes;

	/** Hash to uniquely identify this struct */
	UPROPERTY()
	uint32 FontFaceHash = 0;
};

UCLASS()
class UText3DEngineSubsystem : public UEngineSubsystem, public FSelfRegisteringExec
{
	GENERATED_BODY()

public:
	static UText3DEngineSubsystem* Get();

	void ScheduleCacheCleanup();

	/** Returns the default material for text meshes */
	UMaterial* GetDefaultMaterial() const;

#if WITH_FREETYPE
	FText3DFontFaceCache* FindOrAddCachedFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace);
	FText3DFontFaceCache* FindCachedFontFace(const TSharedPtr<FFreeTypeFace>& InFontFace);
#endif

	FText3DFontFaceCache* FindCachedFontFace(uint32 InFontFaceHash);

	/** Adds the given mesh desc and glyph parameters to the queue */
	void QueueBuildGlyphMeshes(TArray<FText3DBuildGlyphMeshDesc>&& InMeshDescs, const FGlyphMeshParameters& InGlyphMeshParameters);

	/** Returns the number of glyph meshes pending build */
	int32 GetQueuedGlyphMeshBuildCount() const;

	/** Processes the queued mesh descs and glyph parameters across multiple threads */
	void ProcessBuildGlyphMeshes();

private:
	// ~Begin UEngineSubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	// ~End UEngineSubsystem

	// ~Begin FSelfRegisteringExec
	virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* InCmd, FOutputDevice& InAr) override;
	// ~End FSelfRegisteringExec

#if WITH_EDITOR
	/** Called when the UText3DProjectSettings has changed */
	void OnText3DSettingsChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
#endif
	/** Updates the cached default material */
	void UpdateDefaultMaterial();

	bool CleanupCache(float);
	void PrintCache() const;
	void ClearCache();

	UPROPERTY()
	TMap<uint32, FText3DFontFaceCache> CachedFontFaces;

	/** Queued mesh descs to build next */
	TArray<FText3DBuildGlyphMeshDesc> QueuedMeshDescs;

	/** Glyph mesh parameters that the queued mesh descs reference */
	TArray<FGlyphMeshParameters> QueuedGlyphMeshParameters;

	/** the default material to use for text meshes */
	UPROPERTY()
	TObjectPtr<UMaterial> CachedDefaultMaterial;

	FTSTicker::FDelegateHandle CacheCleanupDelegate;

	/** Flag to prevent cleaning while building glyph meshes */
	bool bGlyphMeshBuildInProgress = false;
};
