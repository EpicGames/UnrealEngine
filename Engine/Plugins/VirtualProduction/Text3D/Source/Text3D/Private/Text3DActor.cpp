// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DActor.h"
#include "Text3DComponent.h"

AText3DActor::AText3DActor()
{
	Text3DComponent = CreateDefaultSubobject<UText3DComponent>(TEXT("Text3DComponent"));
	RootComponent = Text3DComponent;
}

#if WITH_EDITOR
FString AText3DActor::GetDefaultActorLabel() const
{
	return TEXT("Text3DActor");
}
#endif
