// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DBuilder.h"
#include "Extensions/Text3DExtensionBase.h"
#include "Logs/Text3DLogs.h"
#include "Renderers/Text3DRendererBase.h"
#include "Text3DComponent.h"
#include "Utilities/Text3DScopedLog.h"

namespace UE::Text3D
{

namespace Private
{

const TCHAR* GetPhaseString(EBuildPhase InPhase)
{
	switch (InPhase)
	{
	case EBuildPhase::NotStarted:         return TEXT("NotStarted");
	case EBuildPhase::PrepareExtensions:  return TEXT("PrepareExtensions");
	case EBuildPhase::PreRendererUpdate:  return TEXT("PreRendererUpdate");
	case EBuildPhase::RendererUpdate:     return TEXT("RendererUpdate");
	case EBuildPhase::PostRendererUpdate: return TEXT("PostRendererUpdate");
	case EBuildPhase::Succeeded:          return TEXT("Succeeded");
	case EBuildPhase::Canceled:           return TEXT("Canceled");
	}
	checkNoEntry();
	return nullptr;
}

} // UE::Text3D::Private

FTextBuilder::FTextBuilder(UText3DComponent* InComponent)
	: Component(InComponent)
{
}

bool FTextBuilder::PrepareFlags(EText3DRendererFlags InFlags)
{
	if (IsInProgress())
	{
		return false;
	}
	EnumAddFlags(Flags, InFlags);
	return true;
}

FString FTextBuilder::GetDebugName() const
{
	return FString::Printf(TEXT("%s (Status: %s)")
		, Component ? *Component->GetReadableName() : TEXT("(none)")
		, *GetDebugStatus());
}

FString FTextBuilder::GetDebugStatus() const
{
	if (IsInRenderingPhases())
	{
		return FString::Printf(TEXT("%s | Flags %0.2d | Current Flag %0.2d")
			, Private::GetPhaseString(Phase)
			, static_cast<int32>(Flags)
			, static_cast<int32>(CurrentFlag));
	}
	return FString::Printf(TEXT("%s | Flags %0.2d")
		, Private::GetPhaseString(Phase)
		, static_cast<int32>(Flags));
}

bool FTextBuilder::IsInProgress() const
{
	return Phase != EBuildPhase::NotStarted && !IsComplete();
}

bool FTextBuilder::IsComplete() const
{
	return Phase == EBuildPhase::Succeeded
		|| Phase == EBuildPhase::Canceled;
}

void FTextBuilder::BlockingBuild()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::BlockingBuild);

	FScopedLog ScopedLog;
	UE_LOGF(LogText3D, VeryVerbose, "%lsPerforming BLOCKING build for '%ls'...", ScopedLog.GetLogPrefix(), *GetDebugName());

	bUseBlockingBuild = true;
	StarvedFrames = 0;
	BudgetEnd.Reset();
	TempPhaseLimit.Reset();
	BuildInternal();
}

void FTextBuilder::Update(const FUpdateParams& InParams)
{
	if (IsComplete())
	{
		return;
	}

	if (Flags == EText3DRendererFlags::None)
	{
		Finish();
		return;
	}

	if (GetUseBlockingBuild())
	{
		BlockingBuild();
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::IncrementalBuild);
	BudgetEnd = InParams.BudgetEnd;
	TempPhaseLimit = InParams.TempMaxUpdatePhase;

	if (!IsWithinPhaseLimit())
	{
		// No work needed if already past the phase limit
		return;
	}

	if (InParams.bBuildTimeExceeded)
	{
		if (StarvedFrames++ < InParams.MaxStarvedFrames)
		{
			return;
		}

		UE_LOGF(LogText3D, VeryVerbose, "%lsIncremental build is out of budget but has starved for more than %d frames. Forcing build..."
			, FScopedLog().GetLogPrefix()
			, InParams.MaxStarvedFrames);
	}

	FScopedLog ScopedLog;
	UE_LOGF(LogText3D, VeryVerbose, "%lsPerforming incremental build for '%ls'...", ScopedLog.GetLogPrefix(), *GetDebugName());
	StarvedFrames = 0;

	const EBuildResult Result = BuildInternal();
	if (Result == EBuildResult::OutOfBudget)
	{
		UE_LOGF(LogText3D, VeryVerbose, "%lsBuild ran out of budget. Continuing next frame", FScopedLog().GetLogPrefix());
		InParams.bBuildTimeExceeded = true;
	}
}

void FTextBuilder::CancelBuild()
{
	Component->CancelBuild();

	Extensions.Reset();
	ExtensionIndex = INDEX_NONE;
	Phase = EBuildPhase::Canceled;
}

void FTextBuilder::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap)
{
	if (UObject* const* TargetObject = InReplacementMap.Find(Component))
	{
		Component = Cast<UText3DComponent>(*TargetObject);
	}

	for (FExtensionElement& Extension : Extensions)
	{
		if (UObject* const* TargetObject = InReplacementMap.Find(Extension.Object))
		{
			Extension.Object = Cast<UText3DExtensionBase>(*TargetObject);
		}
	}
}

void FTextBuilder::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(Component);

	for (FExtensionElement& Extension : Extensions)
	{
		InCollector.AddReferencedObject(Extension.Object);
	}
}

bool FTextBuilder::IsInRenderingPhases() const
{
	return Phase >= EBuildPhase::PreRendererUpdate 
		&& Phase <= EBuildPhase::PostRendererUpdate;
}

EBuildResult FTextBuilder::BuildInternal()
{
	UE_LOGF(LogText3D, VeryVerbose, "%lsPerforming Build. Current Status: %ls", FScopedLog().GetLogPrefix(), *GetDebugStatus());	

	EBuildResult Result;

	switch (Phase)
	{
	case EBuildPhase::NotStarted:
		Result = PrepareComponent();
		if (Extensions.IsEmpty())
		{
			Finish();
			return EBuildResult::BuildCompleted;
		}
		break;

	case EBuildPhase::PrepareExtensions:
		Result = PrepareExtensions();
		break;

	case EBuildPhase::PreRendererUpdate:
		Result = PreRendererUpdate();
		break;

	case EBuildPhase::RendererUpdate:
		Result = RendererUpdate();
		break;

	case EBuildPhase::PostRendererUpdate:
		Result = PostRendererUpdate();
		break;

	case EBuildPhase::Succeeded:
		Finish();
		return EBuildResult::BuildCompleted;

	default:
	case EBuildPhase::Canceled:
		CancelBuild();
		return EBuildResult::BuildCompleted;
	}

	// If work is complete, look to the next phase
	if (Result == EBuildResult::PhaseCompleted)
	{
		return Next();
	}
	return Result;
}

bool FTextBuilder::IsWithinPhaseLimit() const
{
	return !TempPhaseLimit.IsSet() || Phase <= *TempPhaseLimit;
}

bool FTextBuilder::IsWithinBudget() const
{
	// If set, current time is within budget
	const bool bWithinBudget = bSkipNextBudgetCheck || !BudgetEnd.IsSet() || FPlatformTime::Cycles64() <= *BudgetEnd;
	bSkipNextBudgetCheck = false;
	return bWithinBudget;
}

EBuildResult FTextBuilder::IterateExtensions(TFunctionRef<bool(FExtensionElement&)> InFunc)
{
	do
	{
		if (Extensions.IsValidIndex(ExtensionIndex))
		{
			FExtensionElement& Extension = Extensions[ExtensionIndex];
			Extension.bActive = Extension.bActive && InFunc(Extension);
			// Skip budget check if extension was not relevant (no-op)
			bSkipNextBudgetCheck = !Extension.bActive;
			++ExtensionIndex;
		}
		else
		{
			return EBuildResult::PhaseCompleted;
		}
	}
	while (IsWithinBudget());

	return EBuildResult::OutOfBudget;
}

EBuildResult FTextBuilder::PrepareComponent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::PrepareComponent);

	CurrentFlag = 1;
	Extensions.Reset();

	if (!Component->BeginBuild(&Flags))
	{
		return EBuildResult::BuildCompleted;
	}

	// Allocate extension elements
	{
		// Each component has a minimum of 7 extensions, 8 for a single layout effect.
		TArray<UText3DExtensionBase*, TInlineAllocator<8>> DiscoveredExtensions;
		DiscoveredExtensions.Reset();

		Component->GatherExtensions(DiscoveredExtensions);
		if (DiscoveredExtensions.IsEmpty())
		{
			return EBuildResult::BuildCompleted;
		}

		Extensions.Reserve(DiscoveredExtensions.Num());
		for (UText3DExtensionBase* Extension : DiscoveredExtensions)
		{
			FExtensionElement& ExtensionElement = Extensions.AddDefaulted_GetRef();
			ExtensionElement.Object = Extension;
		}
	}

	Extensions.StableSort(
		[](const FExtensionElement& A, const FExtensionElement& B)->bool
		{
			return A.Object->GetUpdatePriority() < B.Object->GetUpdatePriority();
		});

	FlagsLeft = Flags;

	for (FExtensionElement& Extension : Extensions)
	{
		Extension.PreRendererFlagsLeft = Flags;
		Extension.PostRendererFlagsLeft = Flags;
	}

	return EBuildResult::PhaseCompleted;
}

EBuildResult FTextBuilder::PrepareExtensions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::PrepareExtensions);

	return IterateExtensions(
		[Flags=Flags](FExtensionElement& InExtension)
		{
			InExtension.Object->PrepareBuild(Flags);
			return true;
		});
}

EBuildResult FTextBuilder::PreRendererUpdate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::PreRendererUpdate);

	return IterateExtensions(
		[this](FExtensionElement& InExtension)
		{
			const EText3DRendererFlags Flag = static_cast<EText3DRendererFlags>(CurrentFlag);
			if (!EnumHasAnyFlags(InExtension.PreRendererFlagsLeft, Flag))
			{
				return true;
			}
			EnumRemoveFlags(InExtension.PreRendererFlagsLeft, Flag);

			const UE::Text3D::Renderer::FUpdateParameters Parameters
				{
					.UpdateFlags = Flags,
					.CurrentFlag = Flag,
					.bIsLastFlag = InExtension.PreRendererFlagsLeft == EText3DRendererFlags::None
				};
			const EText3DExtensionResult Result = InExtension.Object->PreRendererUpdate(Parameters);
			return Result == EText3DExtensionResult::Active;
		});
}

EBuildResult FTextBuilder::RendererUpdate()
{
	if (!Component->TextRenderer)
	{
		return EBuildResult::PhaseCompleted;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::RendererUpdate);

	const EText3DRendererFlags Flag = static_cast<EText3DRendererFlags>(CurrentFlag);
	if (!EnumHasAnyFlags(FlagsLeft, Flag))
	{
		return EBuildResult::PhaseCompleted;
	}
	EnumRemoveFlags(FlagsLeft, Flag);

	const UE::Text3D::Renderer::FUpdateParameters Parameters
		{
			.UpdateFlags = Flags,
			.CurrentFlag = Flag,
			.bIsLastFlag = FlagsLeft == EText3DRendererFlags::None
		};
	Component->TextRenderer->Update(Parameters);
	return EBuildResult::PhaseCompleted;
}

EBuildResult FTextBuilder::PostRendererUpdate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::PostRendererUpdate);

	const EBuildResult Result = IterateExtensions(
		[this](FExtensionElement& InExtension)
		{
			const EText3DRendererFlags Flag = static_cast<EText3DRendererFlags>(CurrentFlag);
			if (!EnumHasAnyFlags(InExtension.PostRendererFlagsLeft, Flag))
			{
				return true;
			}
			EnumRemoveFlags(InExtension.PostRendererFlagsLeft, Flag);

			const UE::Text3D::Renderer::FUpdateParameters Parameters
				{
					.UpdateFlags = Flags,
					.CurrentFlag = Flag,
					.bIsLastFlag = InExtension.PostRendererFlagsLeft == EText3DRendererFlags::None
				};
			const EText3DExtensionResult Result = InExtension.Object->PostRendererUpdate(Parameters);
			return Result == EText3DExtensionResult::Active;
		});

	if (Result == EBuildResult::PhaseCompleted)
	{
		const uint8 BuildFlags = static_cast<uint8>(Flags);
		const uint8 AllFlags   = static_cast<uint8>(EText3DRendererFlags::All);

		// Get the next flag to iterate
		do
		{
			CurrentFlag <<= 1;

			// If the end is reached, mark as phase completed without looping back.
			if (CurrentFlag > AllFlags)
			{
				return EBuildResult::PhaseCompleted;
			}
		} 
		while ((CurrentFlag & BuildFlags) == 0);

		// Start loop again from PreRendererUpdate. This time with a different renderer flag.
		Phase = static_cast<EBuildPhase>(static_cast<uint8>(EBuildPhase::PreRendererUpdate) - 1);
	}
	return Result;
}

EBuildResult FTextBuilder::Next()
{
	ExtensionIndex = 0;
	Phase = static_cast<EBuildPhase>(static_cast<uint8>(Phase) + 1);

	if (!IsWithinPhaseLimit())
	{
		return EBuildResult::PhaseCompleted;
	}

	if (IsWithinBudget())
	{
		return BuildInternal();
	}

	return EBuildResult::OutOfBudget;
}

void FTextBuilder::Finish()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Text3D::FinishBuild);
	Component->FinishBuild(Flags);

	Extensions.Reset();
	ExtensionIndex = INDEX_NONE;
	Phase = EBuildPhase::Succeeded;

	UE_LOGF(LogText3D, VeryVerbose, "%lsText3D Builder Finished.", FScopedLog().GetLogPrefix());
}

} // UE::Text3D
