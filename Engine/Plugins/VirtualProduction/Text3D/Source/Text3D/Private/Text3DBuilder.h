// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DTypes.h"

class UText3DComponent;
class UText3DExtensionBase;

namespace UE::Text3D
{

/** Enumeration of the different phases of the text building process */
enum class EBuildPhase : uint8
{
	/** Build has not started yet */
	NotStarted,
	/** Prepares the Text3D extensions for building */
	PrepareExtensions,
	/** Calls PreRendererUpdate for the Text3D extensions */
	PreRendererUpdate,
	/** Runs the 'Update' for the Text3D renderer */
	RendererUpdate,
	/** Calls PostRendererUpdate for the Text3D extensions */
	PostRendererUpdate,
	/** Build completed successfully */
	Succeeded,
	/** Build was canceled */
	Canceled,
};

/** The build result for incremental builds  */
enum class EBuildResult : uint8
{
	/** Build ran out of time budget */
	OutOfBudget,
	/** A build phase has completed */
	PhaseCompleted,
	/** Build completed */
	BuildCompleted,
};

/** Manages the building of text for the Text3D Component */
class FTextBuilder
{
	/** Holds the data needed for the Text3D extension during the build process */
	struct FExtensionElement
	{
		/** Text3D extension being built */
		TObjectPtr<UText3DExtensionBase> Object = nullptr;
		/** Remaining flags left for the pre renderer phase.*/
		EText3DRendererFlags PreRendererFlagsLeft = EText3DRendererFlags::None;
		/** Remaining flags left for the post renderer phase. */
		EText3DRendererFlags PostRendererFlagsLeft = EText3DRendererFlags::None;
		/** Whether the extension is active, or has completed and is not needed for the rest of the renderer updates */
		bool bActive = true;
	};

public:
	explicit FTextBuilder(UText3DComponent* InComponent);

	/** 
	 * Adds the given renderer flags to this builder. Should only be done for builders that have not started yet.
	 * @return true if flags were added or false if nothing happened because the builder is already in progress.
	 */
	bool PrepareFlags(EText3DRendererFlags InFlags);

	UText3DComponent* GetComponent() const
	{
		return Component;
	}

	EText3DRendererFlags GetFlags() const
	{
		return Flags;
	}

	EBuildPhase GetPhase() const
	{
		return Phase;
	}

	/** Gets a debug string to identify the builder */
	FString GetDebugName() const;

	/** Gets a debug string to identify the builder */
	FString GetDebugStatus() const;

	/** Returns true if the builder is currently in progress (i.e. between not-started and completed) */
	bool IsInProgress() const;

	/** Returns true if the builder has completed building the component */
	bool IsComplete() const;

	bool GetUseBlockingBuild() const
	{
		return bUseBlockingBuild;
	}

	void SetUseBlockingBuild(bool bInUseBlockingBuild)
	{
		bUseBlockingBuild = bInUseBlockingBuild;
	}

	/** Fully starts / resumes and finishes the build */
	void BlockingBuild();

	struct FUpdateParams
	{
		/** If set, the cycle that the build should stop when past it. If unset, no budget end (analogous to a blocking build) */
		TOptional<uint64> BudgetEnd;
		/** The maximum number of frames the build frame can starve. */
		int32 MaxStarvedFrames = 0;
		/** If set, temporary max build phase that is allowed by the incremental update. If unset, no limit. */
		TOptional<UE::Text3D::EBuildPhase> TempMaxUpdatePhase;
		/** Whether the build time has been exceeded already. Updated by the incremental build calls when this happens. Any non-starving incremental build will be skipped */
		mutable bool bBuildTimeExceeded = false;
	};
	/** Builds the component every call with a budget end and update phase */
	void Update(const FUpdateParams& InParams);

	/** Cancels the build */
	void CancelBuild();

	/** Called by the owning text world subsystem when object replacement has taken place */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap);

	/** Called by the owning text world subsystem to gather objects referenced by this builder */
	void AddReferencedObjects(FReferenceCollector& InCollector);

private:
	/** Whether the builder is currently in rendering phases */
	bool IsInRenderingPhases() const;

	/** Calls the appropriate function depending on the current phase */
	EBuildResult BuildInternal();

	/** Returns true if the current phase of the build is still less than or equal to the limit, or always true if no limit is set */
	bool IsWithinPhaseLimit() const;

	/** Returns true if the current cycles is less than the budgeted end cycle or if no budget has been set at all */
	bool IsWithinBudget() const;

	EBuildResult IterateExtensions(TFunctionRef<bool(FExtensionElement&)> InFunc);

	/** Phase 1 - calls BeginBuild on the component and gathers the component's extensions */
	EBuildResult PrepareComponent();

	/** Phase 2- calls PrepareBuild on the extensions */
	EBuildResult PrepareExtensions();

	/** Phase 3 - calls PreRendererUpdate on the extensions for a single renderer flag */
	EBuildResult PreRendererUpdate();

	/** Phase 4 - calls Update on the component's renderer for a single renderer flag */
	EBuildResult RendererUpdate();

	/**
	 * Phase 5 - calls PostRendererUpdate on the extensions for a single renderer flag 
	 * Once this phase finishes, the next available renderer flag is gotten and if available, build loops back to Phase 3.
	 * If there's no available renderer flag left the build finishes.
	 */
	EBuildResult PostRendererUpdate();

	/** Called to move onto the next phase */
	EBuildResult Next();

	/** Called when the build has completed */
	void Finish();

	/** Component to build */
	TObjectPtr<UText3DComponent> Component;

	/** Extensions being built */
	TArray<FExtensionElement> Extensions;

	/** If set, the cycle that the build should stop when past it  */
	TOptional<uint64> BudgetEnd;

	/** Current extension being updated. Relevant only to certain phases */
	int32 ExtensionIndex = INDEX_NONE;

	/** Number of frames that the incremental build has not been able to execute */
	int32 StarvedFrames = 0;

	/** Renderer flags for the built */
	EText3DRendererFlags Flags = EText3DRendererFlags::None;

	/** Remaining renderer flags used exclusively for the renderer phase */
	EText3DRendererFlags FlagsLeft = EText3DRendererFlags::None;

	/** Current renderer flag. Relevant only to the renderer phases. Does not reset on phase change. */
	uint8 CurrentFlag = 1;

	/** Current build phase */
	EBuildPhase Phase = EBuildPhase::NotStarted;

	/**
	 * Temporary max build phase that is allowed by the incremental update.
	 * Set to a new value every incremental update call.
	 */
	TOptional<EBuildPhase> TempPhaseLimit;

	/** Whether blocking build is used for this builder */
	bool bUseBlockingBuild = false;

	/** Whether budget check should be skipped to avoid querying cycles when unnecessary. Resets every IsWithinBudget() call */
	mutable bool bSkipNextBudgetCheck = false;
};

} // UE::Text3D
