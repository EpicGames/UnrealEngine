// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DComponent.h"

#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "Extensions/Text3DDefaultCharacterExtension.h"
#include "Extensions/Text3DDefaultGeometryExtension.h"
#include "Extensions/Text3DDefaultLayoutExtension.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "Extensions/Text3DDefaultRenderingExtension.h"
#include "Extensions/Text3DDefaultStyleExtension.h"
#include "Extensions/Text3DDefaultTokenExtension.h"
#include "Extensions/Text3DLayoutEffectBase.h"
#include "Fonts/CompositeFont.h"
#include "Internationalization/Regex.h"
#include "Logs/Text3DLogs.h"
#include "Materials/Material.h"
#include "Misc/EnumerateRange.h"
#include "Misc/ScopeExit.h"
#include "Misc/TransactionObjectEvent.h"
#include "Renderers/StaticMeshes/Text3DStaticMeshesRenderer.h"
#include "Renderers/Text3DRendererBase.h"
#include "Settings/Text3DProjectSettings.h"
#include "Subsystems/Text3DBuildSystem.h"
#include "Text3DComponentInstanceData.h"
#include "Text3DComponentVersion.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "Text3D"

#if WITH_EDITOR
UText3DComponent::FOnResolveFontByName UText3DComponent::OnResolveFontByNameDelegate;
#endif

FCustomVersionRegistration GRegisterText3DComponentVersion(FText3DComponentVersion::GUID, static_cast<int32>(FText3DComponentVersion::LatestVersion), TEXT("Text3DComponentVersion"));

UText3DComponent::UText3DComponent()
{
	const UText3DProjectSettings* Settings = UText3DProjectSettings::Get();

	Font = Settings->GetFallbackFont();
	RefreshTypeface();

	Text = LOCTEXT("DefaultText", "Text");

	// Legacy renderer
	TextRendererClass = UText3DStaticMeshesRenderer::StaticClass();

	// Extensions
	TokenExtension = CreateDefaultSubobject<UText3DTokenExtensionBase, UText3DDefaultTokenExtension>(TEXT("TemplateExtension"));
	StyleExtension = CreateDefaultSubobject<UText3DStyleExtensionBase, UText3DDefaultStyleExtension>(TEXT("StyleExtension"));
	MaterialExtension = CreateDefaultSubobject<UText3DMaterialExtensionBase, UText3DDefaultMaterialExtension>(TEXT("MaterialExtension"));
	GeometryExtension = CreateDefaultSubobject<UText3DGeometryExtensionBase, UText3DDefaultGeometryExtension>(TEXT("GeometryExtension"));
	LayoutExtension = CreateDefaultSubobject<UText3DLayoutExtensionBase, UText3DDefaultLayoutExtension>(TEXT("LayoutExtension"));
	RenderingExtension = CreateDefaultSubobject<UText3DRenderingExtensionBase, UText3DDefaultRenderingExtension>(TEXT("RenderingExtension"));
	CharacterExtension = CreateDefaultSubobject<UText3DCharacterExtensionBase, UText3DDefaultCharacterExtension>(TEXT("CharacterExtension"));

	if (!IsTemplate(RF_ClassDefaultObject))
	{
		TransformUpdated.AddUObject(this, &UText3DComponent::OnComponentTransformed);
	}
}

void UText3DComponent::OnComponentCreated()
{
	Super::OnComponentCreated();
	if (TextRenderer)
	{
		TextRenderer->Create();
	}
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DComponent::OnComponentDestroyed(bool bInDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bInDestroyingHierarchy);

	if (UText3DBuildSystem* BuildSystem = UText3DBuildSystem::Get(this))
	{
		BuildSystem->RemoveTextBuild(this);
	}

	if (GUndo)
	{
		if (TextRenderer)
		{
			TextRenderer->Modify();
			TextRenderer->Clear();
			TextRenderer->Destroy();
			TextRenderer = nullptr;
		}
	}

	TransformUpdated.RemoveAll(this);
}

void UText3DComponent::PostLoad()
{
	Super::PostLoad();
	RefreshTypeface();

	// Never perform immediate updates in PostLoad
	const bool bImmediateUpdate = false;
	RequestUpdate(EText3DRendererFlags::All, bImmediateUpdate);
}

void UText3DComponent::RequestUpdate(EText3DRendererFlags InFlags, bool bInImmediateUpdate)
{
	if (InFlags != EText3DRendererFlags::None && !IsTemplate(RF_ClassDefaultObject))
	{
		if (IsRegistered())
		{
			if (UText3DBuildSystem* const BuildSystem = UText3DBuildSystem::Get(this))
			{
				BuildSystem->RequestTextBuild(this, InFlags, bInImmediateUpdate);
			}
		}
		else
		{
			EnumAddFlags(PreRegisterUpdateFlags, InFlags);
		}
	}
}

int32 UText3DComponent::GetTypeFaceIndex() const
{
	int32 TypefaceIndex = INDEX_NONE;

	if (Font)
	{
		TArrayView<const FTypefaceEntry> Fonts = GetAvailableTypefaces();

		for (int32 Index = 0; Index < Fonts.Num(); Index++)
		{
			if (Typeface == Fonts[Index].Name)
			{
				TypefaceIndex = Index;
				break;
			}
		}
	}

	return TypefaceIndex;
}

const FTypefaceEntry* UText3DComponent::GetTypeFaceEntry() const
{
	if (Font)
	{
		TArrayView<const FTypefaceEntry> Fonts = GetAvailableTypefaces();

		for (int32 Index = 0; Index < Fonts.Num(); Index++)
		{
			if (Typeface == Fonts[Index].Name)
			{
				return &Fonts[Index];
			}
		}
	}

	return nullptr;
}

bool UText3DComponent::IsTypefaceAvailable(FName InTypeface) const
{
	for (const FTypefaceEntry& TypefaceElement : GetAvailableTypefaces())
	{
		if (InTypeface == TypefaceElement.Name)
		{
			return true;
		}
	}

	return false;
}

TArrayView<const FTypefaceEntry> UText3DComponent::GetAvailableTypefaces() const
{
	if (Font)
	{
		if (const FCompositeFont* CompositeFont = Font->GetCompositeFont())
		{
			return CompositeFont->DefaultTypeface.Fonts;
		}
	}

	return {};
}

void UText3DComponent::RefreshTypeface()
{
	if (Font)
	{
		TArrayView<const FTypefaceEntry> Fonts = GetAvailableTypefaces();
		for (int32 Index = 0; Index < Fonts.Num(); Index++)
		{
			if (Typeface == Fonts[Index].Name)
			{
				// Typeface stays the same
				return;
			}
		}

		if (!Fonts.IsEmpty())
		{
			Typeface = Fonts[0].Name;
		}
		else
		{
			Typeface = TEXT("");
		}
	}
}

void UText3DComponent::UpdateStatistics()
{
	Statistics = FText3DStatistics();

	const FString WordString = GetFormattedText().ToString();

	const FRegexPattern WordPattern(TEXT("\\S+"));
	FRegexMatcher Matcher(WordPattern, WordString);

	int32 PreviousEndIndex = 0;
	int32 WhitespaceCount = 0;

	while (Matcher.FindNext())
	{
		const FString Word = Matcher.GetCaptureGroup(0);

		if (!Word.IsEmpty())
		{
			FText3DWordStatistics& WordStatistics = Statistics.Words.Add_GetRef(FText3DWordStatistics());
			const int32 MatchBegin = Matcher.GetMatchBeginning();
			const int32 MatchEnd = Matcher.GetMatchEnding();

			WordStatistics.ActualRange = FTextRange(MatchBegin, MatchEnd);

			WhitespaceCount += MatchBegin - PreviousEndIndex;

			WordStatistics.RenderRange = FTextRange(MatchBegin - WhitespaceCount, MatchEnd - WhitespaceCount);

			PreviousEndIndex = MatchEnd;
		}
	}
}

void UText3DComponent::OnComponentTransformed(USceneComponent*, EUpdateTransformFlags, ETeleportType)
{
	/*
	 * TransformUpdated gets broadcasted when component is registered first time so it will have the correct starting value (see UpdateComponentToWorld in UActorComponent::OnRegister)
	 * Further calls affecting transform value, will have the proper zero scaled status
	 */
	const bool bIsZeroScaled = FMath::IsNearlyZero(GetComponentScale().GetMin(), UE_KINDA_SMALL_NUMBER);

	if (bIsZeroScaled)
	{
		bZeroScaled = true;
	}

	// Only update when scale was zero and is not anymore
	if (bZeroScaled && !bIsZeroScaled)
	{
		RequestUpdate(EText3DRendererFlags::Layout | EText3DRendererFlags::Material);
		bZeroScaled = false;
	}
}

bool UText3DComponent::IsCompiledFromBlueprint() const
{
	for (const UObject* Object = this; Object != nullptr && Object->GetOuter(); Object = Object->GetOuter())
	{
		if (Object->GetClass()->HasAllClassFlags(CLASS_CompiledFromBlueprint))
		{
			return true;
		}

		if (Object->IsA<AActor>())
		{
			break;
		}
	}

	return false;
}

void UText3DComponent::OnRegister()
{
	Super::OnRegister();
	RequestUpdate(PreRegisterUpdateFlags);
}

void UText3DComponent::OnUnregister()
{
	Super::OnUnregister();
}

void UText3DComponent::PostApplyToComponent()
{
	Super::PostApplyToComponent();
	RequestUpdate(EText3DRendererFlags::All);
}

TStructOnScope<FActorComponentInstanceData> UText3DComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FText3DComponentInstanceData>(this);
}

void UText3DComponent::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FText3DComponentVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FText3DComponentVersion::GUID);

	if (Version < FText3DComponentVersion::LatestVersion)
	{
		UE_LOGF(LogText3D, Log, "Text3D : Migrating from %i to %i version", Version, FText3DComponentVersion::LatestVersion)

		if (Version < FText3DComponentVersion::Extensions)
		{
#if WITH_EDITORONLY_DATA
			PRAGMA_DISABLE_DEPRECATION_WARNINGS

			if (MaterialExtension)
			{
				MaterialExtension->SetMaterial(EText3DGroupType::Front, FrontMaterial_DEPRECATED);
				MaterialExtension->SetMaterial(EText3DGroupType::Back, BackMaterial_DEPRECATED);
				MaterialExtension->SetMaterial(EText3DGroupType::Extrude, ExtrudeMaterial_DEPRECATED);
				MaterialExtension->SetMaterial(EText3DGroupType::Bevel, BevelMaterial_DEPRECATED);
			}

			if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
			{
				DefaultGeometryExtension->SetExtrude(Extrude_DEPRECATED);
				DefaultGeometryExtension->SetBevel(Bevel_DEPRECATED);
				DefaultGeometryExtension->SetBevelType(BevelType_DEPRECATED);
				DefaultGeometryExtension->SetBevelSegments(BevelSegments_DEPRECATED);
				DefaultGeometryExtension->SetUseOutline(bOutline_DEPRECATED);
				DefaultGeometryExtension->SetOutline(OutlineExpand_DEPRECATED);
			}

			if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
			{
				DefaultLayoutExtension->SetHorizontalAlignment(HorizontalAlignment_DEPRECATED);
				DefaultLayoutExtension->SetVerticalAlignment(VerticalAlignment_DEPRECATED);
				DefaultLayoutExtension->SetTracking(Kerning_DEPRECATED);
				DefaultLayoutExtension->SetLineSpacing(LineSpacing_DEPRECATED);
				DefaultLayoutExtension->SetWordSpacing(WordSpacing_DEPRECATED);
				DefaultLayoutExtension->SetUseMaxWidth(bHasMaxWidth_DEPRECATED);
				DefaultLayoutExtension->SetMaxWidth(MaxWidth_DEPRECATED);
				DefaultLayoutExtension->SetMaxWidthBehavior(MaxWidthHandling_DEPRECATED);
				DefaultLayoutExtension->SetUseMaxHeight(bHasMaxHeight_DEPRECATED);
				DefaultLayoutExtension->SetMaxHeight(MaxHeight_DEPRECATED);
				DefaultLayoutExtension->SetScaleProportionally(bScaleProportionally_DEPRECATED);
			}
			
			if (UText3DDefaultRenderingExtension* DefaultRenderingExtension = GetCastedRenderingExtension<UText3DDefaultRenderingExtension>())
			{
				DefaultRenderingExtension->SetCastShadow(bCastShadow_DEPRECATED);
			}

			PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
		}
	}
}

void UText3DComponent::PostEditImport()
{
	Super::PostEditImport();
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	RequestUpdate(EText3DRendererFlags::All);
}

#if WITH_EDITOR
void UText3DComponent::PostCDOCompiled(const FPostCDOCompiledContext& InContext)
{
	Super::PostCDOCompiled(InContext);
}

void UText3DComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	static const TSet<FName> FontPropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UText3DComponent, Font),
		GET_MEMBER_NAME_CHECKED(UText3DComponent, Typeface)
	};

	static const TSet<FName> TextPropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UText3DComponent, Text),
		GET_MEMBER_NAME_CHECKED(UText3DComponent, bEnforceUpperCase),
		GET_MEMBER_NAME_CHECKED(UText3DComponent, FontSize)
	};

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DComponent, TextRendererClass))
	{
		OnTextRendererClassChanged();
	}
	else if (TextPropertyNames.Contains(MemberPropertyName))
	{
		OnTextChanged();
	}
	else if (FontPropertyNames.Contains(MemberPropertyName))
	{
		OnFontPropertiesChanged();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UText3DComponent, LayoutEffects))
	{
		EText3DRendererFlags Flags = EText3DRendererFlags::Layout;
		EnumAddFlags(Flags, EText3DRendererFlags::Material);
		RequestUpdate(Flags);
	}
}

void UText3DComponent::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		RequestUpdate(EText3DRendererFlags::All);
	}
}

void UText3DComponent::SetFontByName(const FString& InFontName)
{
	if (OnResolveFontByNameDelegate.IsBound())
	{
		if (UFont* ResolvedFont = OnResolveFontByNameDelegate.Execute(InFontName))
		{
			SetFont(ResolvedFont);
		}
	}
}

void UText3DComponent::ForceUpdateText()
{
	const FText PrevText = GetText();
	SetText(FText::GetEmpty());
	SetText(PrevText);
}

void UText3DComponent::OpenProjectSettings()
{
	if (const UText3DProjectSettings* TextSettings = UText3DProjectSettings::Get())
	{
		TextSettings->OpenEditorSettingsWindow();
	}
}
#endif

const FText& UText3DComponent::GetFormattedText() const
{
	if (const UText3DTokenExtensionBase* Extension = GetTokenExtension())
	{
		return Extension->GetFormattedText();
	}

	return Text;
}

void UText3DComponent::SetText(const FText& Value)
{
	if (Text.EqualTo(Value))
	{
		return;
	}

	Text = Value;
	OnTextChanged();
}

void UText3DComponent::SetEnforceUpperCase(bool bInEnforceUpperCase)
{
	if (bEnforceUpperCase == bInEnforceUpperCase)
	{
		return;
	}

	bEnforceUpperCase = bInEnforceUpperCase;
	OnTextChanged();
}

void UText3DComponent::SetFont(UFont* InFont)
{
	if (!InFont || Font == InFont)
	{
		return;
	}
	
	Font = InFont;
	OnFontPropertiesChanged();
}

void UText3DComponent::SetFontSize(float InSize)
{
	InSize = FMath::Max(0.1, InSize);
	if (FMath::IsNearlyEqual(InSize, FontSize))
	{
		return;
	}

	FontSize = InSize;
	OnFontPropertiesChanged();
}

float UText3DComponent::GetFontSize() const
{
	return FontSize;
}

void UText3DComponent::SetHasOutline(const bool bValue)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetUseOutline(bValue);
	}
}

void UText3DComponent::SetOutlineExpand(const float Value)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetOutline(Value);
	}
}

void UText3DComponent::SetExtrude(const float Value)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetExtrude(Value);
	}
}

void UText3DComponent::SetBevel(const float Value)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetBevel(Value);
	}
}

void UText3DComponent::SetBevelType(const EText3DBevelType Value)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetBevelType(Value);
	}
}

void UText3DComponent::SetBevelSegments(const int32 Value)
{
	if (UText3DDefaultGeometryExtension* DefaultGeometryExtension = GetCastedGeometryExtension<UText3DDefaultGeometryExtension>())
	{
		DefaultGeometryExtension->SetBevelSegments(Value);
	}
}

UMaterialInterface* UText3DComponent::GetBackMaterial() const
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Back;
		return MaterialExtension->GetMaterial(Parameters);
	}

	return nullptr;
}

UMaterialInterface* UText3DComponent::GetExtrudeMaterial() const
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Extrude;
		return MaterialExtension->GetMaterial(Parameters);
	}

	return nullptr;
}

UMaterialInterface* UText3DComponent::GetBevelMaterial() const
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Bevel;
		return MaterialExtension->GetMaterial(Parameters);
	}

	return nullptr;
}

UMaterialInterface* UText3DComponent::GetFrontMaterial() const
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Front;
		return MaterialExtension->GetMaterial(Parameters);
	}

	return nullptr;
}

void UText3DComponent::SetFrontMaterial(UMaterialInterface* Value)
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Front;
		return MaterialExtension->SetMaterial(Parameters, Value);
	}
}

void UText3DComponent::SetBevelMaterial(UMaterialInterface* Value)
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Bevel;
		return MaterialExtension->SetMaterial(Parameters, Value);
	}
}

void UText3DComponent::SetExtrudeMaterial(UMaterialInterface* Value)
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Extrude;
		return MaterialExtension->SetMaterial(Parameters, Value);
	}
}

void UText3DComponent::SetBackMaterial(UMaterialInterface* Value)
{
	if (MaterialExtension)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = EText3DGroupType::Back;
		return MaterialExtension->SetMaterial(Parameters, Value);
	}
}

float UText3DComponent::GetKerning() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetTracking();
	}

	return 0.f;
}

UText3DRendererBase* UText3DComponent::GetTextRenderer() const
{
	return TextRenderer;
}

UText3DRendererBase* UText3DComponent::GetTextRenderer(TSubclassOf<UText3DRendererBase> InRendererClass) const
{
	if (!TextRenderer || !InRendererClass.Get())
	{
		return nullptr;
	}
	
	return TextRenderer->IsA(InRendererClass.Get()) ? TextRenderer : nullptr;
}

UText3DTokenExtensionBase* UText3DComponent::GetTokenExtension(TSubclassOf<UText3DTokenExtensionBase> InExtensionClass)
{
	if (!TokenExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}

	return TokenExtension->IsA(InExtensionClass.Get()) ? TokenExtension : nullptr;
}

UText3DStyleExtensionBase* UText3DComponent::GetStyleExtension(TSubclassOf<UText3DStyleExtensionBase> InExtensionClass)
{
	if (!StyleExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}

	return StyleExtension->IsA(InExtensionClass.Get()) ? StyleExtension : nullptr;
}

UText3DLayoutExtensionBase* UText3DComponent::GetLayoutExtension(TSubclassOf<UText3DLayoutExtensionBase> InExtensionClass)
{
	if (!LayoutExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}
	
	return LayoutExtension->IsA(InExtensionClass.Get()) ? LayoutExtension : nullptr;
}

UText3DMaterialExtensionBase* UText3DComponent::GetMaterialExtension(TSubclassOf<UText3DMaterialExtensionBase> InExtensionClass)
{
	if (!MaterialExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}
	
	return MaterialExtension->IsA(InExtensionClass.Get()) ? MaterialExtension : nullptr;
}

UText3DGeometryExtensionBase* UText3DComponent::GetGeometryExtension(TSubclassOf<UText3DGeometryExtensionBase> InExtensionClass)
{
	if (!GeometryExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}
	
	return GeometryExtension->IsA(InExtensionClass.Get()) ? GeometryExtension : nullptr;
}

UText3DRenderingExtensionBase* UText3DComponent::GetRenderingExtension(TSubclassOf<UText3DRenderingExtensionBase> InExtensionClass)
{
	if (!RenderingExtension || !InExtensionClass.Get())
	{
		return nullptr;
	}
	
	return RenderingExtension->IsA(InExtensionClass.Get()) ? RenderingExtension : nullptr;
}

TArray<UText3DLayoutEffectBase*> UText3DComponent::GetLayoutEffects(TSubclassOf<UText3DLayoutEffectBase> InEffectClass)
{
	TArray<UText3DLayoutEffectBase*> Effects;

	if (!InEffectClass.Get())
	{
		return Effects;
	}

	Effects.Reserve(LayoutEffects.Num());
	for (UText3DLayoutEffectBase* LayoutEffect : LayoutEffects)
	{
		if (LayoutEffect && LayoutEffect->IsA(InEffectClass))
		{
			Effects.Add(LayoutEffect);
		}
	}

	return Effects;
}

uint16 UText3DComponent::GetCharacterCount() const
{
	return CharacterExtension->GetCharacterCount();
}

UText3DCharacterBase* UText3DComponent::GetCharacter(uint16 InCharacterIndex) const
{
	return CharacterExtension->GetCharacter(InCharacterIndex);
}

void UText3DComponent::ForEachCharacter(const TFunctionRef<void(UText3DCharacterBase*, uint16, uint16)>& InFunctor) const
{
	const uint16 TotalCount = GetCharacterCount();
	for (TConstEnumerateRef<UText3DCharacterBase*> CharacterRef : EnumerateRange(CharacterExtension->GetCharacters()))
	{
		InFunctor(*CharacterRef, CharacterRef.GetIndex(), TotalCount);
	}
}

void UText3DComponent::SetKerning(const float Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetTracking(Value);
	}
}

float UText3DComponent::GetLineSpacing() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetLineSpacing();
	}

	return 0.f;
}

void UText3DComponent::SetLineSpacing(const float Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetLineSpacing(Value);
	}
}

float UText3DComponent::GetWordSpacing() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetWordSpacing();
	}

	return 0.f;
}

void UText3DComponent::SetWordSpacing(const float Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetWordSpacing(Value);
	}
}

EText3DHorizontalTextAlignment UText3DComponent::GetHorizontalAlignment() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetHorizontalAlignment();
	}

	return EText3DHorizontalTextAlignment::Left;
}

void UText3DComponent::SetHorizontalAlignment(const EText3DHorizontalTextAlignment Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetHorizontalAlignment(Value);
	}
}

EText3DVerticalTextAlignment UText3DComponent::GetVerticalAlignment() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetVerticalAlignment();
	}

	return EText3DVerticalTextAlignment::FirstLine;
}

void UText3DComponent::SetVerticalAlignment(const EText3DVerticalTextAlignment Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetVerticalAlignment(Value);
	}
}

bool UText3DComponent::HasMaxWidth() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetUseMaxWidth();
	}

	return false;
}

void UText3DComponent::SetHasMaxWidth(const bool Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetUseMaxWidth(Value);
	}
}

float UText3DComponent::GetMaxWidth() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetMaxWidth();
	}

	return 0.f;
}

void UText3DComponent::SetMaxWidth(const float Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetMaxWidth(Value);
	}
}

EText3DMaxWidthHandling UText3DComponent::GetMaxWidthHandling() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetMaxWidthBehavior();
	}

	return EText3DMaxWidthHandling::Scale;
}

void UText3DComponent::SetMaxWidthHandling(const EText3DMaxWidthHandling Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetMaxWidthBehavior(Value);
	}
}

bool UText3DComponent::HasMaxHeight() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetUseMaxHeight();
	}

	return false;
}

void UText3DComponent::SetHasMaxHeight(const bool Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetUseMaxHeight(Value);
	}
}

float UText3DComponent::GetMaxHeight() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetMaxHeight();
	}

	return 0.f;
}

void UText3DComponent::SetMaxHeight(const float Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetMaxHeight(Value);
	}
}

bool UText3DComponent::ScalesProportionally() const
{
	if (const UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		return DefaultLayoutExtension->GetScalesProportionally();
	}

	return false;
}

void UText3DComponent::SetScaleProportionally(const bool Value)
{
	if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
	{
		DefaultLayoutExtension->SetScaleProportionally(Value);
	}
}

bool UText3DComponent::CastsShadow() const
{
	if (RenderingExtension)
	{
		return RenderingExtension->GetTextCastShadow();
	}

	return false;
}

void UText3DComponent::SetCastShadow(bool NewCastShadow)
{
	if (UText3DDefaultRenderingExtension* DefaultRenderingExtension = GetCastedRenderingExtension<UText3DDefaultRenderingExtension>())
	{
		DefaultRenderingExtension->SetCastShadow(NewCastShadow);
	}
}

void UText3DComponent::SetTypeface(const FName InTypeface)
{
	if (Typeface == InTypeface || !IsTypefaceAvailable(InTypeface))
	{
		return;
	}

	Typeface = InTypeface;
	OnFontPropertiesChanged();
}

void UText3DComponent::SetTextRendererClass(const TSubclassOf<UText3DRendererBase>& InClass)
{
	if (!TextRendererClass.Get()
		|| TextRendererClass == InClass)
	{
		return;
	}

	TextRendererClass = InClass;
	OnTextRendererClassChanged();
}

void UText3DComponent::OnTextRendererClassChanged()
{
	if (IsTemplate(RF_ClassDefaultObject))
	{
		return;
	}

	if (TextRenderer && TextRenderer->GetClass() == TextRendererClass && TextRenderer->GetOuter() == this)
	{
		return;
	}

	if (!TextRendererClass)
	{
		if (TextRenderer)
		{
			TextRendererClass = TextRenderer->GetClass();
		}
		else
		{
			TextRendererClass = UText3DStaticMeshesRenderer::StaticClass();
		}
	}

	const UText3DRendererBase* RendererCDO = TextRendererClass->GetDefaultObject<UText3DRendererBase>();

	if (!RendererCDO)
	{
		return;
	}

	FName OldRendererName = NAME_None;

	auto TrashObject = [](UObject* InObject)
		{
			if (!InObject)
			{
				return;
			}
			UPackage* const TransientPackage = GetTransientPackage();
			const FString UniqueObjectName = MakeUniqueObjectName(TransientPackage, InObject->GetClass(), InObject->GetFName()).ToString();
			InObject->Rename(*UniqueObjectName, TransientPackage, REN_DontCreateRedirectors|REN_NonTransactional|REN_DoNotDirty);
			InObject->MarkAsGarbage();
			InObject = nullptr;
		};

	if (TextRenderer)
	{
		OldRendererName = TextRenderer->GetFriendlyName();

		TextRenderer->Clear();
		TextRenderer->Destroy();
		TrashObject(TextRenderer);
		TextRenderer = nullptr;
	}

	const FName RendererName = RendererCDO->GetFriendlyName();

	// Trash any existing renderer, instead of re-using it. It is likely in an invalid state.
	TrashObject(FindObject<UText3DRendererBase>(this, RendererName.ToString()));
	TextRenderer = NewObject<UText3DRendererBase>(this, TextRendererClass.Get(), RendererName, RF_Public);

	const AActor* ContextActor = GetOwner();
	UE_LOGF(LogText3D, Log, "%ls : Text3D renderer changed : old %ls - new %ls", !!ContextActor ? *ContextActor->GetActorNameOrLabel() : TEXT("Invalid owner"), *OldRendererName.ToString(), *TextRenderer->GetFriendlyName().ToString());

	if (!bIsUpdatingText)
	{
		RequestUpdate(EText3DRendererFlags::All);
	}
}

void UText3DComponent::OnTextChanged()
{
	RequestUpdate(EText3DRendererFlags::All);
}

void UText3DComponent::OnFontPropertiesChanged()
{
	RefreshTypeface();
	RequestUpdate(EText3DRendererFlags::All);
}

TArray<FName> UText3DComponent::GetTypefaceNames() const
{
	TArray<FName> TypefaceNames;

	if (Font)
	{
		for (const FTypefaceEntry& TypeFaceFont : GetAvailableTypefaces())
		{
			TypefaceNames.Add(TypeFaceFont.Name);
		}
	}

	return TypefaceNames;
}

bool UText3DComponent::GatherExtensions(TAdderReserverRef<UText3DExtensionBase*> OutExtensions) const
{
	// Let's make sure text extension subobjects are valid and usable !
	if (!TokenExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Token extension, cannot proceed !")
		return false;
	}

	if (!StyleExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Style extension, cannot proceed !")
		return false;
	}

	if (!CharacterExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Character extension, cannot proceed !")
		return false;
	}

	if (!GeometryExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Geometry extension, cannot proceed !")
		return false;
	}

	if (!LayoutExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Layout extension, cannot proceed !")
		return false;
	}

	if (!MaterialExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Material extension, cannot proceed !")
		return false;
	}

	if (!RenderingExtension)
	{
		UE_LOGF(LogText3D, Warning, "Invalid Text3D Rendering extension, cannot proceed !")
		return false;
	}

	OutExtensions.Reserve(7 + LayoutEffects.Num());
	OutExtensions.Add(TokenExtension);
	OutExtensions.Add(StyleExtension);
	OutExtensions.Add(CharacterExtension);
	OutExtensions.Add(GeometryExtension);
	OutExtensions.Add(LayoutExtension);
	OutExtensions.Add(MaterialExtension);
	OutExtensions.Add(RenderingExtension);

	// Add optional extensions for layout effects
	for (UText3DLayoutEffectBase* LayoutEffect : LayoutEffects)
	{
		if (LayoutEffect)
		{
			OutExtensions.Add(LayoutEffect);
		}
	}

	return true;
}

bool UText3DComponent::BeginBuild(TNotNull<EText3DRendererFlags*> InOutFlags)
{
	bIsUpdatingText = true;
	TextPreUpdateDelegate.Broadcast(this, *InOutFlags);

	// If only updating materials, no need to prepare TextRenderer, or Font because the material extension is mostly self contained.
	// TODO: asking extensions whether TextRenderer/etc is needed when building with specific flags?
	if (*InOutFlags == EText3DRendererFlags::Material)
	{
		return true;
	}

	if (!TextRenderer || TextRenderer->GetClass() != TextRendererClass || TextRenderer->GetOuter() != this)
	{
		OnTextRendererClassChanged();
		if (!TextRenderer)
		{
			UE_LOGF(LogText3D, Warning, "Text3D renderer is not valid, cannot proceed");
			return false;
		}
	}

	if (!Font)
	{
		if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
		{
			Font = Text3DSettings->GetFallbackFont();
		}

		if (!Font)
		{
			UE_LOGF(LogText3D, Warning, "Font object is not valid, Fallback font is not defined in project settings, cannot proceed");
			return false;
		}
	}

	if (!TextRenderer->IsInitialized())
	{
		TextRenderer->Create();
		*InOutFlags = EText3DRendererFlags::All;
	}
	return true;
}

void UText3DComponent::FinishBuild(EText3DRendererFlags InFlags)
{
	if (EnumHasAnyFlags(InFlags,EText3DRendererFlags::Geometry))
	{
		UpdateStatistics();
		TextGeneratedNativeDelegate.Broadcast();
		TextGeneratedDelegate.Broadcast();
	}

	if (EnumHasAnyFlags(InFlags,EText3DRendererFlags::Visibility))
	{
		MarkRenderStateDirty();
	}

	EnumRemoveFlags(PreRegisterUpdateFlags, InFlags);
	TextPostUpdateDelegate.Broadcast(this, InFlags);
	bIsUpdatingText = false;
}

void UText3DComponent::CancelBuild()
{
	bIsUpdatingText = false;
}

void UText3DComponent::SetUseBlockingBuild(bool bInUseBlockingBuild)
{
	bUseBlockingBuild = bInUseBlockingBuild;
}

void UText3DComponent::OnVisibilityChanged()
{
	Super::OnVisibilityChanged();

	RequestUpdate(EText3DRendererFlags::Visibility);
}

void UText3DComponent::OnHiddenInGameChanged()
{
	Super::OnHiddenInGameChanged();

	RequestUpdate(EText3DRendererFlags::Visibility);
}

void UText3DComponent::FormatText(FText& InOutText) const
{
	if (bEnforceUpperCase)
	{
		InOutText = InOutText.ToUpper();
	}
}

void UText3DComponent::GetBounds(FVector& Origin, FVector& BoxExtent) const
{
	FBox Box(ForceInit);

	if (TextRenderer)
	{
		Box = TextRenderer->GetBounds();
	}

	Box.GetCenterAndExtents(Origin, BoxExtent);
}

FBox UText3DComponent::GetBounds() const
{
	if (TextRenderer)
	{
		return TextRenderer->GetBounds();
	}

	return FBox(ForceInit);
}

#undef LOCTEXT_NAMESPACE
