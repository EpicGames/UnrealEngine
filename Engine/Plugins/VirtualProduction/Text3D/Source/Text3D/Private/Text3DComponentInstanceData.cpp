// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DComponentInstanceData.h"
#include "Algo/ForEach.h"
#include "Extensions/Text3DCharacterExtensionBase.h"
#include "Extensions/Text3DGeometryExtensionBase.h"
#include "Extensions/Text3DLayoutEffectBase.h"
#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "Extensions/Text3DRenderingExtensionBase.h"
#include "Extensions/Text3DStyleExtensionBase.h"
#include "Extensions/Text3DTokenExtensionBase.h"
#include "Renderers/Text3DRendererBase.h"
#include "Text3DComponent.h"
#include "UObject/Package.h"

FText3DComponentInstanceData::FText3DComponentInstanceData(const UText3DComponent* InSourceComponent)
	: FActorComponentInstanceData(InSourceComponent)
{
	TextRenderer       = InSourceComponent->GetTextRenderer();
	CharacterExtension = InSourceComponent->GetCharacterExtension();
	GeometryExtension  = InSourceComponent->GetGeometryExtension();
	LayoutExtension    = InSourceComponent->GetLayoutExtension();
	MaterialExtension  = InSourceComponent->GetMaterialExtension();
	RenderingExtension = InSourceComponent->GetRenderingExtension();
	StyleExtension     = InSourceComponent->GetStyleExtension();
	TokenExtension     = InSourceComponent->GetTokenExtension();
	LayoutEffects      = InSourceComponent->GetLayoutEffects();
}

bool FText3DComponentInstanceData::ContainsData() const
{
	return true;
}

void FText3DComponentInstanceData::ApplyToComponent(UActorComponent* InComponent, const ECacheApplyPhase InCacheApplyPhase)
{
	Super::ApplyToComponent(InComponent, InCacheApplyPhase);

	UText3DComponent* const Component = CastChecked<UText3DComponent>(InComponent);

	Component->TextRenderer       = TextRenderer;
	Component->CharacterExtension = CharacterExtension;
	Component->GeometryExtension  = GeometryExtension;
	Component->LayoutExtension    = LayoutExtension;
	Component->MaterialExtension  = MaterialExtension;
	Component->RenderingExtension = RenderingExtension;
	Component->StyleExtension     = StyleExtension;
	Component->TokenExtension     = TokenExtension;
	Component->LayoutEffects      = LayoutEffects;

	UObject* const TransientPackage = GetTransientPackage();

	// Ensure that the object is outered to this component, discarding any new default replacement
	auto PatchObject = [Component, TransientPackage](UObject* InObject)
		{
			// The object is outered to the old component. Rename it to be outered to this new one
			if (InObject && InObject->GetOuter() != Component)
			{
				// discard any existing object that has the name of the object so that there's no collision
				if (UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Component, *InObject->GetName()))
				{
					const FName UniqueName = MakeUniqueObjectName(TransientPackage, ExistingObject->GetClass(), *(TEXT("TRASH_") + ExistingObject->GetName()));
					ExistingObject->Rename(*UniqueName.ToString(), TransientPackage, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
					ExistingObject->MarkAsGarbage();
				}
				InObject->Rename(nullptr, Component, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
		};

	PatchObject(Component->TextRenderer);
	PatchObject(Component->CharacterExtension);
	PatchObject(Component->GeometryExtension);
	PatchObject(Component->LayoutExtension);
	PatchObject(Component->MaterialExtension);
	PatchObject(Component->RenderingExtension);
	PatchObject(Component->StyleExtension);
	PatchObject(Component->TokenExtension);
	Algo::ForEach(Component->LayoutEffects, PatchObject);

	// Re-initialize TextRenderer
	if (Component->TextRenderer)
	{
		Component->TextRenderer->bInitialized = false;
	}
}

void FText3DComponentInstanceData::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FActorComponentInstanceData::AddReferencedObjects(InCollector);
	InCollector.AddReferencedObject(TextRenderer);
	InCollector.AddReferencedObject(CharacterExtension);
	InCollector.AddReferencedObject(GeometryExtension);
	InCollector.AddReferencedObject(LayoutExtension);
	InCollector.AddReferencedObject(MaterialExtension);
	InCollector.AddReferencedObject(RenderingExtension);
	InCollector.AddReferencedObject(StyleExtension);
	InCollector.AddReferencedObject(TokenExtension);
	InCollector.AddReferencedObjects(LayoutEffects);
}
