// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentInstanceDataCache.h"
#include "Text3DComponentInstanceData.generated.h"

class UText3DCharacterExtensionBase;
class UText3DComponent;
class UText3DGeometryExtensionBase;
class UText3DLayoutEffectBase;
class UText3DLayoutExtensionBase;
class UText3DMaterialExtensionBase;
class UText3DRendererBase;
class UText3DRenderingExtensionBase;
class UText3DStyleExtensionBase;
class UText3DTokenExtensionBase;

USTRUCT()
struct FText3DComponentInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()

	FText3DComponentInstanceData() = default;
	explicit FText3DComponentInstanceData(const UText3DComponent* InSourceComponent);

	//~ Begin FActorComponentInstanceData
	virtual bool ContainsData() const override;
	virtual void ApplyToComponent(UActorComponent* InComponent, const ECacheApplyPhase InCacheApplyPhase) override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FActorComponentInstanceData

private:
	/** Source text renderer */
	TObjectPtr<UText3DRendererBase> TextRenderer = nullptr;

	/** Source extensions */
	TObjectPtr<UText3DCharacterExtensionBase> CharacterExtension = nullptr;
	TObjectPtr<UText3DGeometryExtensionBase> GeometryExtension = nullptr;
	TObjectPtr<UText3DLayoutExtensionBase> LayoutExtension = nullptr;
	TObjectPtr<UText3DMaterialExtensionBase> MaterialExtension = nullptr;
	TObjectPtr<UText3DRenderingExtensionBase> RenderingExtension = nullptr;
	TObjectPtr<UText3DStyleExtensionBase> StyleExtension = nullptr;
	TObjectPtr<UText3DTokenExtensionBase> TokenExtension = nullptr;
	TArray<TObjectPtr<UText3DLayoutEffectBase>> LayoutEffects;
};
