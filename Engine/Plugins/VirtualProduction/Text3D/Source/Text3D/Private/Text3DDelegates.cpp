// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DDelegates.h"

namespace UE::Text3D
{
namespace Private
{

TMulticastDelegate<void(const FText3DBuildContext&)>& GetPreRendererUpdateDelegate()
{
	static TMulticastDelegate<void(const FText3DBuildContext&)> StaticDelegate;
	return StaticDelegate;
}

TMulticastDelegate<void(const FText3DBuildContext&)>& GetPostRendererUpdateDelegate()
{
	static TMulticastDelegate<void(const FText3DBuildContext&)> StaticDelegate;
	return StaticDelegate;
}

} // UE::Text3D::Private

FText3DBuildContext::FText3DBuildContext(const Renderer::FUpdateParameters& InUpdateParameters)
	: UpdateFlags(InUpdateParameters.UpdateFlags)
{
}

void BroadcastPreRendererUpdate(const FText3DBuildContext& InBuildContext)
{
	Private::GetPreRendererUpdateDelegate().Broadcast(InBuildContext);
}

void BroadcastPostRendererUpdate(const FText3DBuildContext& InBuildContext)
{
	Private::GetPostRendererUpdateDelegate().Broadcast(InBuildContext);
}

TMulticastDelegateRegistration<void(const FText3DBuildContext&)>& OnText3DPreRendererUpdate()
{
	return Private::GetPreRendererUpdateDelegate();
}

TMulticastDelegateRegistration<void(const FText3DBuildContext&)>& OnText3DPostRendererUpdate()
{
	return Private::GetPostRendererUpdateDelegate();
}

} // UE::Text3D
