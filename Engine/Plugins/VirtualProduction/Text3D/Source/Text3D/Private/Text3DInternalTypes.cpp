// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DInternalTypes.h"
#include "Containers/Array.h"
#include "UObject/NameTypes.h"

namespace UE::Text3D::Material
{

TConstArrayView<FName> GetSlotNames()
{
	static const TArray<FName> SlotNames = {
		TEXT("Front"),
		TEXT("Bevel"),
		TEXT("Extrude"),
		TEXT("Back"),
	};
	return SlotNames;
}

} // UE::Text3D::Material
