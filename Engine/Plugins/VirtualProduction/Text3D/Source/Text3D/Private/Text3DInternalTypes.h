// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

class FName;

namespace UE::Text3D::Material
{
	TConstArrayView<FName> GetSlotNames();

} // UE::Text3D::Material
