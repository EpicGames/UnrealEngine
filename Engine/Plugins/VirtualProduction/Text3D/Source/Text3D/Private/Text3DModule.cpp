// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DModule.h"

#include "Logs/Text3DLogs.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogText3D);

IMPLEMENT_MODULE(FText3DModule, Text3D);

FText3DModule::FText3DModule()
{
#if WITH_FREETYPE
	FreeTypeLib = nullptr;
#endif
}

void FText3DModule::StartupModule()
{
	// Ensure that GeometryMask is loaded for Text3D (content-only dependency).
	FModuleManager::Get().LoadModule(TEXT("GeometryMask"));

#if WITH_FREETYPE
	FT_Init_FreeType(&FreeTypeLib);
#endif
}

void FText3DModule::ShutdownModule()
{
#if WITH_FREETYPE
	FT_Done_FreeType(FreeTypeLib);
	FreeTypeLib = nullptr;
#endif
}

#if WITH_FREETYPE
FT_Library FText3DModule::GetFreeTypeLibrary()
{
	const FText3DModule& Instance = FModuleManager::LoadModuleChecked<FText3DModule>("Text3D");
	return Instance.FreeTypeLib;
}
#endif