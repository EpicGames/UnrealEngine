// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#if WITH_FREETYPE

#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
#define generic __identifier(generic)
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD

THIRD_PARTY_INCLUDES_START
#include "ft2build.h"
#include FT_FREETYPE_H
#include FT_ADVANCES_H
THIRD_PARTY_INCLUDES_END

#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
#undef generic
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD

#endif

class FText3DModule final : public IModuleInterface
{
public:
	FText3DModule();

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

#if WITH_FREETYPE
	static FT_Library GetFreeTypeLibrary();

private:
	FT_Library FreeTypeLib;
#endif
};
