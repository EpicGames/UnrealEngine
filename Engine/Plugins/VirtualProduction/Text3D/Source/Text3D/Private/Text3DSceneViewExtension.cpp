// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DSceneViewExtension.h"
#include "IText3DBuildSystem.h"
#include "SceneView.h"
#include "Utilities/Text3DUtilities.h"

namespace UE::Text3D
{

namespace Private
{

TAutoConsoleVariable<bool> CVarHidePrimitivesWhileBuilding(TEXT("Text3D.Build.HidePrimitivesWhileBuilding")
	, false
	, TEXT("If true, primitives managed by Text3D components that are being built will be hidden until completion."));

} // UE::Text3D::Private

void FText3DSceneViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	if (!InViewFamily.Scene)
	{
		return;
	}
	if (const IText3DBuildSystemInterface* const BuildSystem = Utilities::FindBuildSystem(InViewFamily.Scene->GetWorld()))
	{
		BuildSystem->HidePrimitivesBeingBuilt(InView);
	}
}

bool FText3DSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& InContext) const
{
	const bool bEnabled = Private::CVarHidePrimitivesWhileBuilding.GetValueOnGameThread();
	if (!bEnabled)
	{
		return false;
	}

	const IText3DBuildSystemInterface* const BuildSystem = Utilities::FindBuildSystem(InContext.GetWorld());
	return BuildSystem && BuildSystem->IsBuildInProgress();
}

} // UE::Text3D
