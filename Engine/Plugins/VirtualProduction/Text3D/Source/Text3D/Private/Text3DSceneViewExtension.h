// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

namespace UE::Text3D
{

/** Scene view extensions to hide any primitive managed by Text3D that is still under construction */
class FText3DSceneViewExtension : public FSceneViewExtensionBase
{
public:
	explicit FText3DSceneViewExtension(const FAutoRegister& InAutoRegister)
		: FSceneViewExtensionBase(InAutoRegister)
	{
	}

protected:
	//~ Begin FSceneViewExtensionBase
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	//~ End FSceneViewExtensionBase
};

} // UE::Text3D
