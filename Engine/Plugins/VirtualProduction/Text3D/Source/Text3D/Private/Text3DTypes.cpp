// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DTypes.h"
#include "Logs/Text3DLogs.h"
#include "Subsystems/Text3DEngineSubsystem.h"

namespace UE::Text3D::Geometry
{
	FCachedFontFaceGlyphHandle::FCachedFontFaceGlyphHandle(const FText3DFontFaceCache& InFontFaceCache, uint32 InGlyphIndex, const FGlyphMeshParameters& InGlyphMeshParameters)
	{
		GlyphIndex = InGlyphIndex;
		FontFaceHash = GetTypeHash(InFontFaceCache);
		GlyphMeshHash = FText3DFontFaceCache::GetGlyphMeshHash(InGlyphIndex, InGlyphMeshParameters);

		if (FText3DCachedMesh* CachedMesh = Resolve())
		{
			CachedMesh->RefCount++;
		}
		else
		{
			UE_LOGF(LogText3D, Warning, "Constructed glyph handle does not point to a valid entry! No ref counting will occur. Glyph Index: %d, Font Face Hash: %u, Glyph Mesh Hash: %u. Font Face: %ls"
				, GlyphIndex
				, FontFaceHash
				, GlyphMeshHash
				, *InFontFaceCache.ToDebugString());
		}
	}

	FCachedFontFaceGlyphHandle::FCachedFontFaceGlyphHandle(const FCachedFontFaceGlyphHandle& InOther)
	{
		*this = InOther;
	}

	FCachedFontFaceGlyphHandle::FCachedFontFaceGlyphHandle(FCachedFontFaceGlyphHandle&& InOther)
	{
		*this = MoveTemp(InOther);
	}

	FCachedFontFaceGlyphHandle::~FCachedFontFaceGlyphHandle()
	{
		Unset();
	}

	FCachedFontFaceGlyphHandle& FCachedFontFaceGlyphHandle::operator=(const FCachedFontFaceGlyphHandle& InOther)
	{
		if (this != &InOther)
		{
			Unset();
			FontFaceHash = InOther.FontFaceHash;
			GlyphIndex = InOther.GlyphIndex;
			GlyphMeshHash = InOther.GlyphMeshHash;
			if (FText3DCachedMesh* CachedMesh = Resolve())
			{
				CachedMesh->RefCount++;
			}
		}
		return *this;
	}

	FCachedFontFaceGlyphHandle& FCachedFontFaceGlyphHandle::operator=(FCachedFontFaceGlyphHandle&& InOther)
	{
		if (this != &InOther)
		{
			Unset();
			FontFaceHash = InOther.FontFaceHash;
			GlyphIndex = InOther.GlyphIndex;
			GlyphMeshHash = InOther.GlyphMeshHash;
			InOther.FontFaceHash = 0;
			InOther.GlyphIndex = 0;
			InOther.GlyphMeshHash = 0;
		}
		return *this;
	}

	void FCachedFontFaceGlyphHandle::Unset()
	{
		if (FText3DCachedMesh* CachedMesh = Resolve())
		{
			if (CachedMesh->RefCount == 0 || --CachedMesh->RefCount == 0)
			{
				UText3DEngineSubsystem::Get()->ScheduleCacheCleanup();
			}
		}

		FontFaceHash = 0;
		GlyphMeshHash = 0;
		GlyphIndex = 0;
	}

	const FText3DCachedMesh* FCachedFontFaceGlyphHandle::Resolve() const
	{
		if (IsValid())
		{
			if (UText3DEngineSubsystem* Text3DSubsystem = UText3DEngineSubsystem::Get())
			{
				if (const FText3DFontFaceCache* CachedFontFace = Text3DSubsystem->FindCachedFontFace(FontFaceHash))
				{
					if (const FText3DCachedMesh* CachedMesh = CachedFontFace->FindGlyphMesh(GlyphMeshHash))
					{
						return CachedMesh;
					}
				}
			}
		}

		return nullptr;
	}

	FText3DCachedMesh* FCachedFontFaceGlyphHandle::Resolve()
	{
		if (IsValid())
		{
			if (UText3DEngineSubsystem* Text3DSubsystem = UText3DEngineSubsystem::Get())
			{
				if (FText3DFontFaceCache* CachedFontFace = Text3DSubsystem->FindCachedFontFace(FontFaceHash))
				{
					if (FText3DCachedMesh* CachedMesh = CachedFontFace->FindGlyphMesh(GlyphMeshHash))
					{
						return CachedMesh;
					}
				}
			}
		}

		return nullptr;
	}
}
