// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tokens/Text3DTokenBase.h"

#include "Text3DComponent.h"

#if WITH_EDITOR
void UText3DTokenBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UText3DTokenBase, TokenName)
		|| MemberName == GET_MEMBER_NAME_CHECKED(UText3DTokenBase, Content))
	{
		OnTokenPropertiesChanged();
	}
}
#endif

void UText3DTokenBase::SetTokenName(FName InName)
{
	if (InName == TokenName)
	{
		return;
	}

	TokenName = InName;
	OnTokenPropertiesChanged();
}

void UText3DTokenBase::SetContent(const FText& InContent)
{
	if (InContent.EqualTo(Content))
	{
		return;
	}

	Content = InContent;
	OnTokenPropertiesChanged();
}

void UText3DTokenBase::OnTokenPropertiesChanged()
{
	if (!TokenName.IsNone())
	{
		if (UText3DComponent* Text3DComponent = GetTypedOuter<UText3DComponent>())
		{
			Text3DComponent->RequestUpdate(EText3DRendererFlags::All);
		}
	}
}

void UText3DTokenBase::CollectTokens(FFormatNamedArguments& InNamedArguments)
{
	if (!TokenName.IsNone())
	{
		const FString TokenPattern = TokenName.ToString();
		const FString EscapeTokenPattern = FString::Printf(TEXT("\\%s\\"), *TokenPattern);
		const FString EscapeTokenMarker = FString::Printf(TEXT("{%s}"), *TokenPattern);

		// Replaces {\token\} by {token} and {token} by its content
		InNamedArguments.Add(*EscapeTokenPattern, FText::FromString(EscapeTokenMarker));
		InNamedArguments.Add(*TokenPattern, Content);
	}
}
