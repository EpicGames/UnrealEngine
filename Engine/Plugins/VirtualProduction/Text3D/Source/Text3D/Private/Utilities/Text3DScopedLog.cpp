// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DScopedLog.h"

#if UE_TEXT3D_LOG_SCOPE_ENABLED
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Templates/Tuple.h"

namespace UE::Text3D
{

namespace Private
{

static int32 ScopeLevel = -1;

} // UE::Text3D::Private

FScopedLog::FScopedLog()
{
	++Private::ScopeLevel;
}

FScopedLog::~FScopedLog()
{
	--Private::ScopeLevel;
}

const TCHAR* FScopedLog::GetLogPrefix() const
{
	if (Private::ScopeLevel <= 0)
	{
		static TPair<uint64, FString> CachedFrameCounter(0, FString());
		if (CachedFrameCounter.Key != GFrameCounter)
		{
			CachedFrameCounter.Value = FString::Printf(TEXT("[%llu] "), GFrameCounter);
		}
		return *CachedFrameCounter.Value;
	}
	constexpr int32 NumSpacesPerScopeLevel = 3;
	// FCString::Spc returns a ptr to a static string of spaces
	return FCString::Spc(Private::ScopeLevel * NumSpacesPerScopeLevel);
}

} // UE::Text3D

#endif // UE_TEXT3D_LOG_SCOPE_ENABLED
