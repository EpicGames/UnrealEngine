// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Platform.h"

#define UE_TEXT3D_LOG_SCOPE_ENABLED 1 && !NO_LOGGING

namespace UE::Text3D
{

#if UE_TEXT3D_LOG_SCOPE_ENABLED
/** Handles adding and removing 'scope levels' for log clarity */
struct FScopedLog
{
	FScopedLog();
	~FScopedLog();

	const TCHAR* GetLogPrefix() const;
};
#else
struct FScopedLog
{
	const TCHAR* GetLogPrefix() const
	{
		return TEXT("");
	}
};
#endif // UE_TEXT3D_LOG_SCOPE_ENABLED

} // UE::Text3D
