// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/Text3DUtilities.h"

#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Fonts/CompositeFont.h"
#include "Misc/Paths.h"
#include "Subsystems/Text3DBuildSystem.h"
#include "Text3DComponent.h"
#include "Text3DModule.h"

#if WITH_EDITOR
#include "Components/StaticMeshComponent.h"
#include "Dialogs/DlgPickPath.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "IMeshMergeUtilities.h"
#include "Materials/Material.h"
#include "MeshMergeModule.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "Modules/ModuleManager.h"
#include "Settings/Text3DProjectSettings.h"
#include "UObject/Package.h"
#endif

#if WITH_FREETYPE
THIRD_PARTY_INCLUDES_START
#include "ft2build.h"
#include FT_FREETYPE_H
THIRD_PARTY_INCLUDES_END
#endif

#define LOCTEXT_NAMESPACE "Text3DUtilities"

IText3DBuildSystemInterface* UE::Text3D::Utilities::FindBuildSystem(const UObject* InWorldContextObject)
{
	if (!GEngine)
	{
		return nullptr;
	}
	if (const UWorld* const World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return World->GetSubsystem<UText3DBuildSystem>();
	}
	return nullptr;
}

bool UE::Text3D::Utilities::Font::GetSanitizeFontName(const UFont* InFont, FString& OutFontName)
{
	if (GetFontName(InFont, OutFontName))
	{
		SanitizeFontName(OutFontName);
		return true;
	}

	return false;
}

bool UE::Text3D::Utilities::Font::GetFontName(const UFont* InFont, FString& OutFontName)
{
	if (!IsValid(InFont))
	{
		return false;
	}
	
	FString FontAssetName;
	FString FontImportName = InFont->ImportOptions.FontName;

	if (InFont->GetFName() == NAME_None)
	{
		FontAssetName = InFont->LegacyFontName.ToString();
	}
	else
	{
		FontAssetName = InFont->GetName();
	}

	// Roboto fonts are actually from the Arial family and their import name is "Arial", so we try to list them as well
	// this will likely lead to missing spaces in their names
	if (FontAssetName.Contains(TEXT("Roboto")) || FontImportName == TEXT("Arial"))
	{
		OutFontName = FontAssetName;
	}
	else
	{
		OutFontName = FontImportName;
	}

	return !OutFontName.IsEmpty();
}

void UE::Text3D::Utilities::Font::SanitizeFontName(FString& InOutFontName)
{
	// spaces would be also handled by code below, but spaces used to be removed
	// so let's to this anyway in order to avoid any potential issues with new vs. old imported fonts

	const TCHAR* InvalidObjectChar = INVALID_OBJECTNAME_CHARACTERS;
	while (*InvalidObjectChar)
	{
		InOutFontName.ReplaceCharInline(*InvalidObjectChar, TCHAR(' '), ESearchCase::CaseSensitive);
		++InvalidObjectChar;
	}

	const TCHAR* InvalidPackageChar = INVALID_LONGPACKAGE_CHARACTERS;
	while (*InvalidPackageChar)
	{
		InOutFontName.ReplaceCharInline(*InvalidPackageChar, TCHAR(' '), ESearchCase::CaseSensitive);
		++InvalidPackageChar;
	}

	InOutFontName.RemoveSpacesInline();
}

bool UE::Text3D::Utilities::Font::GetFontStyle(const UFont* InFont, EText3DFontStyleFlags& OutFontStyleFlags)
{
#if WITH_FREETYPE
	if (!IsValid(InFont))
	{
		return false;
	}

	const FCompositeFont* const CompositeFont = InFont->GetCompositeFont();
	if (!CompositeFont || CompositeFont->DefaultTypeface.Fonts.IsEmpty())
	{
		return false;
	}
	
	OutFontStyleFlags = EText3DFontStyleFlags::None;

	for (const FTypefaceEntry& TypefaceEntry : CompositeFont->DefaultTypeface.Fonts)
	{
		const FFontFaceDataConstPtr FaceData = TypefaceEntry.Font.GetFontFaceData();

		if (FaceData.IsValid() && FaceData->HasData() && FaceData->GetData().Num() > 0)
		{
			const TArray<uint8> Data = FaceData->GetData();

			FT_Face FreeTypeFace = nullptr;
			const FT_Error Error = FT_New_Memory_Face(FText3DModule::GetFreeTypeLibrary(), Data.GetData(), Data.Num(), 0, &FreeTypeFace);
			if (Error)
			{
				// We assume the face is null if the function errored
				ensureAlwaysMsgf(FreeTypeFace == nullptr, TEXT("FT_New_Memory_Face failed but also returned a non-null FT_Face. This is unexpected and may leak memory!"));
				FreeTypeFace = nullptr;
			}

			if (FreeTypeFace)
			{
				if (FreeTypeFace->style_flags & FT_STYLE_FLAG_ITALIC)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Italic);
				}

				if (FreeTypeFace->style_flags & FT_STYLE_FLAG_BOLD)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Bold);
				}

				if (FreeTypeFace->style_flags & FT_FACE_FLAG_FIXED_WIDTH)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Monospace);
				}
				
				FT_Done_Face(FreeTypeFace);
			}
		}
	}

	{
		int32 Height;

		int32 SpaceWidth;
		InFont->GetStringHeightAndWidth(TEXT(" "), Height, SpaceWidth);

		int32 LWidth;
		InFont->GetStringHeightAndWidth(TEXT("l"), Height, LWidth);

		int32 WWidth;
		InFont->GetStringHeightAndWidth(TEXT("W"), Height, WWidth);

		if ((SpaceWidth == LWidth) && (SpaceWidth == WWidth))
		{
			EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Monospace);
		}
	}

	return true;
#else
	UE_LOGF(LogText3D, Warning, "FreeType is not available, cannot proceed without it")
	return false;
#endif
}

bool UE::Text3D::Utilities::Font::GetFontStyle(const FText3DFontFamily& InFontFamily, EText3DFontStyleFlags& OutFontStyleFlags)
{
	OutFontStyleFlags = EText3DFontStyleFlags::None;

#if WITH_FREETYPE
	if (InFontFamily.FontFacePaths.IsEmpty())
	{
		return false;
	}

	for (const TPair<FString, FString>& FontFace : InFontFamily.FontFacePaths)
	{
		if (FPaths::FileExists(FontFace.Value))
		{
			FT_Face FreeTypeFace;
			FT_New_Face(FText3DModule::GetFreeTypeLibrary(), TCHAR_TO_ANSI(*FontFace.Value), 0, &FreeTypeFace);

			if (FreeTypeFace)
			{
				if (FreeTypeFace->style_flags & FT_STYLE_FLAG_ITALIC)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Italic);
				}

				if (FreeTypeFace->style_flags & FT_STYLE_FLAG_BOLD)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Bold);
				}

				if (FreeTypeFace->style_flags & FT_FACE_FLAG_FIXED_WIDTH)
				{
					EnumAddFlags(OutFontStyleFlags, EText3DFontStyleFlags::Monospace);
				}

				FT_Done_Face(FreeTypeFace);
			}
		}
	}

	return true;
#else
	UE_LOGF(LogText3D, Warning, "FreeType is not available, cannot proceed without it")
	return false;
#endif
}

bool UE::Text3D::Utilities::Font::GetFontFaces(const UFont* InFont, TArray<UFontFace*>& OutFontFaces)
{
	OutFontFaces.Empty();
	
	if (!IsValid(InFont))
	{
		return false;
	}

	for (const FTypefaceEntry& TypefaceEntry : InFont->GetCompositeFont()->DefaultTypeface.Fonts)
	{
		if (const UFontFace* FontFace = Cast<UFontFace>(TypefaceEntry.Font.GetFontFaceAsset()))
		{
			OutFontFaces.Add(const_cast<UFontFace*>(FontFace));
		}
	}

	return !OutFontFaces.IsEmpty();
}

#if WITH_EDITOR
bool UE::Text3D::Utilities::Conversion::PickAssetPath(const FString& InDefaultPath, FString& OutPickedPath)
{
	TSharedPtr<SDlgPickPath> DialogWidget = SNew(SDlgPickPath)
		.Title(LOCTEXT("PickAssetsLocation", "Choose Asset(s) Location"))
		.DefaultPath(FText::FromString(InDefaultPath));

	if (DialogWidget->ShowModal() != EAppReturnType::Ok)
	{
		OutPickedPath = TEXT("");
		return false;
	}

	OutPickedPath = DialogWidget->GetPath().ToString() + TEXT("/");
	return true;
}

AStaticMeshActor* UE::Text3D::Utilities::Conversion::ConvertToStaticMesh(const UText3DComponent* InComponent)
{
	AStaticMeshActor* NewActor = nullptr;

	if (!IsValid(InComponent))
	{
		return NewActor;
	}

	UWorld* World = InComponent->GetWorld();
	AActor* Owner = InComponent->GetOwner();

	if (!IsValid(World)
		|| !IsValid(Owner)
		|| Owner->bIsEditorPreviewActor
		)
	{
		return NewActor;
	}

	const UPackage* CurrentPackage = InComponent->GetPackage();
	FString PackagePath;
	if (!CurrentPackage || !PickAssetPath(CurrentPackage->GetLoadedPath().GetPackageName(), PackagePath))
	{
		return NewActor;
	}

	const FString AssetPath = PackagePath + Owner->GetActorNameOrLabel() + TEXT("_Merged_") + FString::FromInt(Owner->GetUniqueID());

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	Owner->GetComponents(PrimitiveComponents);

	if (PrimitiveComponents.IsEmpty())
	{
		return NewActor;
	}

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	FMeshMergingSettings MergeSettings;
	MergeSettings.bBakeVertexDataToMesh = false;
	MergeSettings.bMergeMaterials = false;
	MergeSettings.bMergeEquivalentMaterials = false;
	MergeSettings.bUseVertexDataForBakingMaterial = false;
	TArray<UObject*> OutObjects;
	FVector OutLocation;
	MeshMergeUtilities.MergeComponentsToStaticMesh(PrimitiveComponents, nullptr, MergeSettings, nullptr, nullptr, AssetPath, OutObjects, OutLocation, 1.f, false);

	if (OutObjects.IsEmpty())
	{
		return NewActor;
	}

	UStaticMesh* Mesh = Cast<UStaticMesh>(OutObjects[0]);

	if (!Mesh)
	{
		return NewActor;
	}

	const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get();
	for (int32 Index = 0; Index < Mesh->GetStaticMaterials().Num(); Index++)
	{
		Mesh->SetMaterial(Index, Text3DSettings->GetDefaultMaterial());
	}

	// Spawn attached actor with same flags as this actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = Owner;
	SpawnParameters.ObjectFlags = Owner->GetFlags();
	SpawnParameters.bTemporaryEditorActor = false;

	const FTransform& Transform = InComponent->GetComponentTransform();
	NewActor = World->SpawnActor<AStaticMeshActor>(AStaticMeshActor::StaticClass(), Transform, SpawnParameters);

	if (!NewActor)
	{
		return NewActor;
	}

	NewActor->SetMobility(EComponentMobility::Movable);

	UStaticMeshComponent* StaticMeshComponent = NewActor->GetStaticMeshComponent();
	StaticMeshComponent->SetStaticMesh(Mesh);

	return NewActor;
}
#endif

#undef LOCTEXT_NAMESPACE