// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "Text3DTypes.h"
#include "UObject/Object.h"
#include "Text3DCharacterBase.generated.h"

/** Holds data for a single character in Text3D */
UCLASS(MinimalAPI, AutoExpandCategories=(Character))
class UText3DCharacterBase : public UObject
{
	GENERATED_BODY()

public:
	static TEXT3D_API FName GetRelativeLocationPropertyName();
	static TEXT3D_API FName GetRelativeRotationPropertyName();
	static TEXT3D_API FName GetRelativeScalePropertyName();
	static TEXT3D_API FName GetVisiblePropertyName();

	TEXT3D_API FTransform& GetTransform(bool bInReset);

#if WITH_EDITORONLY_DATA
	void SetCharacter(const FString& InCharacter)
	{
		Character = InCharacter;
	}

	const FString& GetCharacter() const
	{
		return Character;
	}
#endif

	const UE::Text3D::Geometry::FCachedFontFaceGlyphHandle& GetFontFaceGlyphHandle() const;
	void SetFontFaceGlyphHandle(const UE::Text3D::Geometry::FCachedFontFaceGlyphHandle& InHandle);

	void SetStyleTag(FName InStyle);
	FName GetStyleTag() const
	{
		return StyleTag;
	}

	uint32 GetGlyphIndex() const;
	const FText3DCachedMesh* GetGlyphMesh() const;
	void GetGlyphMeshBoundsAndOffset(FBox& OutBounds, FVector& OutOffset) const;

	UFUNCTION()
	TEXT3D_API void SetRelativeLocation(const FVector& InLocation);
	const FVector& GetRelativeLocation() const
	{
		return RelativeLocation;
	}

	UFUNCTION()
	TEXT3D_API void SetRelativeRotation(const FRotator& InRotation);
	const FRotator& GetRelativeRotation() const
	{
		return RelativeRotation;
	}

	UFUNCTION()
	TEXT3D_API void SetRelativeScale(const FVector& InScale);
	const FVector& GetRelativeScale() const
	{
		return RelativeScale;
	}

	UFUNCTION()
	TEXT3D_API void SetVisibility(bool bInVisibility);
	bool GetVisibility() const
	{
		return bVisible;
	}

	/** Get character custom kerning */
	virtual float GetCharacterKerning() const
	{
		return 0.f;
	}

	/** Reset properties to their initial state when character is recycled */
	TEXT3D_API virtual void ResetCharacterState();

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	void OnCharacterDataChanged(EText3DRendererFlags InFlags) const;

#if WITH_EDITORONLY_DATA
	/** Used for TitleProperty of the array to display this instead of the index */
	UPROPERTY(VisibleAnywhere, Category="Character", Transient, meta=(EditCondition="false", EditConditionHides, AllowPrivateAccess="true"))
	FString Character;
#endif

	UPROPERTY(EditAnywhere, Setter, Getter, Category="Character", meta=(AllowPrivateAccess = "true"))
	FVector RelativeLocation = FVector::ZeroVector;
	
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Character", meta=(AllowPrivateAccess = "true"))
	FRotator RelativeRotation = FRotator::ZeroRotator;
	
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Character", meta=(Delta="0.05", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", ClampMin="0", AllowPrivateAccess = "true"))
	FVector RelativeScale = FVector::OneVector;

	UPROPERTY(EditAnywhere, Setter="SetVisibility", Getter="GetVisibility", Category="Character", meta=(AllowPrivateAccess = "true"))
	bool bVisible = true;

	/** Tag used to identify the custom style applied to this character */
	UPROPERTY(VisibleAnywhere, Setter, Getter, Category="Character")
	FName StyleTag = NAME_None;

	/** Final transform after all extensions are applied */
	FTransform Transform = FTransform::Identity;

	/** Handle to the current glyph mesh this character represents */
	UE::Text3D::Geometry::FCachedFontFaceGlyphHandle FontFaceGlyphHandle;
};
