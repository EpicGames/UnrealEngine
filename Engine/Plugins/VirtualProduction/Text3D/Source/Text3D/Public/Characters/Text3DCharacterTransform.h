// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Text3DTypes.h"
#include "Text3DCharacterTransform.generated.h"

class UText3DLayoutTransformEffect;
class UText3DComponent;

/**
 * This class should no longer be used with Text3D
 * It is only kept to migrate properties to the newer system
 * See UText3DLayoutTransformEffect that can be added in section Text3DComponent/Layout/LayoutEffects
 */
UCLASS(MinimalAPI, ClassGroup=(Text3D), HideCategories = (Collision, Tags, Activation, Cooking, Rendering, Physics, Mobility, LOD, AssetUserData, Navigation, Transform))
class UText3DCharacterTransform : public USceneComponent
{
	GENERATED_BODY()

public:	
	UText3DCharacterTransform();

	bool GetLocationEnabled() const
	{
		return bLocationEnabled;
	}

	float GetLocationProgress() const
	{
		return LocationProgress;
	}

	EText3DCharacterEffectOrder GetLocationOrder() const
	{
		return LocationOrder;
	}

	float GetLocationRange() const
	{
		return LocationRange;
	}

	FVector GetLocationDistance() const
	{
		return LocationDistance;
	}

	bool GetRotationEnabled() const
	{
		return bRotateEnabled;
	}

	float GetRotationProgress() const
	{
		return RotateProgress;
	}

	EText3DCharacterEffectOrder GetRotationOrder() const
	{
		return RotateOrder;
	}

	float GetRotationRange() const
	{
		return RotateRange;
	}

	FRotator GetRotationBegin() const
	{
		return RotateBegin;
	}

	FRotator GetRotationEnd() const
	{
		return RotateEnd;
	}

	bool GetScaleEnabled() const
	{
		return bScaleEnabled;
	}

	float GetScaleProgress() const
	{
		return ScaleProgress;
	}

	EText3DCharacterEffectOrder GetScaleOrder() const
	{
		return ScaleOrder;
	}

	float GetScaleRange() const
	{
		return ScaleRange;
	}

	FVector GetScaleBegin() const
	{
		return ScaleBegin;
	}

	FVector GetScaleEnd() const
	{
		return ScaleEnd;
	}

	// Location

	UFUNCTION(BlueprintCallable, Category = "Location")
	TEXT3D_API void SetLocationEnabled(bool bEnabled);

	UFUNCTION(BlueprintCallable, Category = "Location")
	TEXT3D_API void SetLocationProgress(float Progress);

	UFUNCTION(BlueprintCallable, Category = "Location")
	TEXT3D_API void SetLocationOrder(EText3DCharacterEffectOrder Order);

	UFUNCTION(BlueprintCallable, Category = "Location")
	TEXT3D_API void SetLocationRange(float Range);

	UFUNCTION(BlueprintCallable, Category = "Location")
	TEXT3D_API void SetLocationDistance(FVector Distance);

	// Rotation

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateEnabled(bool bEnabled);

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateProgress(float Progress);

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateOrder(EText3DCharacterEffectOrder Order);

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateRange(float Range);

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateBegin(FRotator Value);

	UFUNCTION(BlueprintCallable, Category = "Rotate")
	TEXT3D_API void SetRotateEnd(FRotator Value);

	// Scale

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleEnabled(bool bEnabled);

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleProgress(float Progress);

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleOrder(EText3DCharacterEffectOrder Order);

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleRange(float Range);

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleBegin(FVector Value);

	UFUNCTION(BlueprintCallable, Category = "Scale")
	TEXT3D_API void SetScaleEnd(FVector Value);

protected:
	UText3DComponent* GetText3DComponent() const;
	UText3DLayoutTransformEffect* GetText3DLayoutTransformEffect() const;

	//~ Begin UActorComponent
	virtual void OnComponentCreated() override;
	//~ End UActorComponent

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	void PrintDeprecationLog() const;

	// Location

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetLocationEnabled, Category = "Location")
	bool bLocationEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetLocationProgress, Category = "Location", meta = (EditCondition = "bLocationEnabled", ClampMin = 0, ClampMax = 100))
	float LocationProgress;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetLocationOrder, Category = "Location", meta = (EditCondition = "bLocationEnabled"))
	EText3DCharacterEffectOrder LocationOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetLocationRange, Category = "Location", meta = (EditCondition = "bLocationEnabled", ClampMin = 0, ClampMax = 100))
	float LocationRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetLocationDistance, Category = "Location", meta = (EditCondition = "bLocationEnabled"))
	FVector LocationDistance;

	// Rotate

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateEnabled, Category = "Rotate")
	bool bRotateEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateProgress, Category = "Rotate", meta = (EditCondition = "bRotateEnabled", ClampMin = 0, ClampMax = 100))
	float RotateProgress;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateOrder, Category = "Rotate", meta = (EditCondition = "bRotateEnabled"))
	EText3DCharacterEffectOrder RotateOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateRange, Category = "Rotate", meta = (EditCondition = "bRotateEnabled", ClampMin = 0, ClampMax = 100))
	float RotateRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateBegin, Category = "Rotate", meta = (EditCondition = "bRotateEnabled"))
	FRotator RotateBegin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetRotateEnd, Category = "Rotate", meta = (EditCondition = "bRotateEnabled"))
	FRotator RotateEnd;

	// Scale

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleEnabled, Category = "Scale")
	bool bScaleEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleProgress, Category = "Scale", meta = (EditCondition = "bScaleEnabled", ClampMin = 0, ClampMax = 100))
	float ScaleProgress;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleOrder, Category = "Scale", meta = (EditCondition = "bScaleEnabled"))
	EText3DCharacterEffectOrder ScaleOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleRange, Category = "Scale", meta = (EditCondition = "bScaleEnabled", ClampMin = 0, ClampMax = 100))
	float ScaleRange;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleBegin, Category = "Scale", meta = (EditCondition = "bScaleEnabled", ClampMin = 0))
	FVector ScaleBegin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetScaleEnd, Category = "Scale", meta = (EditCondition = "bScaleEnabled", ClampMin = 0))
	FVector ScaleEnd;
};
