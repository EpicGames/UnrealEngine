// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DCharacterBase.h"
#include "Text3DTypes.h"
#include "Text3DDefaultCharacter.generated.h"

/** Holds data for a single character in Text3D */
UCLASS(MinimalAPI, AutoExpandCategories=(Character))
class UText3DDefaultCharacter : public UText3DCharacterBase
{
	GENERATED_BODY()

public:
	static TEXT3D_API FName GetKerningPropertyName();

	TEXT3D_API void SetKerning(float InKerning);
	float GetKerning() const
	{
		return Kerning;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DCharacterBase
	virtual float GetCharacterKerning() const override;
	virtual void ResetCharacterState() override;
	//~ End UText3DCharacterBase

	/** Kerning adjusts the space between this character */
	UPROPERTY(EditAnywhere, Getter, Setter, Category="Character", meta=(AllowPrivateAccess = "true"))
	float Kerning = 0.f;
};
