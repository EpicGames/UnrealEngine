// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DCharacterExtensionBase.generated.h"

class UText3DCharacterBase;

/** Extension that handles character data for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DCharacterExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	/** Get the amount of character */
	virtual uint16 GetCharacterCount() const
	{
		return 0;
	}

	/** Get per character data */
	virtual UText3DCharacterBase* GetCharacter(uint16 InCharacter) const
	{
		return nullptr;
	}

	/** Get all character data */
	virtual TConstArrayView<UText3DCharacterBase*> GetCharacters() const
	{
		return {};
	}

	/** Allocate visible characters to store data */
	virtual void AllocateCharacters(uint16 InCount) {}

#if WITH_EDITORONLY_DATA
	/** Used for detail customization within editor */
	UPROPERTY(EditInstanceOnly, Transient, Category="Character", meta=(TextCharacterSelector, AllowPrivateAccess=true))
	uint16 TextCharacterIndex = 0;
#endif
};