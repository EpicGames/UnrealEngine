// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DCharacterExtensionBase.h"
#include "Text3DDefaultCharacterExtension.generated.h"

class UText3DDefaultCharacter;

UCLASS(MinimalAPI)
class UText3DDefaultCharacterExtension : public UText3DCharacterExtensionBase
{
	GENERATED_BODY()
	
protected:
	//~ Begin UText3DGeometryExtensionBase
	virtual uint16 GetCharacterCount() const override;
	virtual UText3DCharacterBase* GetCharacter(uint16 InCharacter) const override;
	virtual TConstArrayView<UText3DCharacterBase*> GetCharacters() const override;
	virtual void AllocateCharacters(uint16 InCount) override;
	//~ End UText3DGeometryExtensionBase

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	/** Allocate character data by using pool or creating new object */
	void AllocateTextCharacters(uint16 InCharacterCount);

	/** Characters composing the active text */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UText3DCharacterBase>> TextCharacters;

	/** Pool of characters to reuse when text changes to avoid creating new objects */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<TObjectPtr<UText3DCharacterBase>> TextCharactersPool;
};