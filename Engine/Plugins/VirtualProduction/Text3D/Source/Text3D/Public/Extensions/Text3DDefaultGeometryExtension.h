// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DGeometryExtensionBase.h"
#include "Text3DDefaultGeometryExtension.generated.h"

UCLASS(MinimalAPI)
class UText3DDefaultGeometryExtension : public UText3DGeometryExtensionBase
{
	GENERATED_BODY()

public:
	/** Get the text extrusion size and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	float GetExtrude() const
	{
		return Extrude;
	}

	/** Set the text extrusion size and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetExtrude(const float Value);

	/** Get the 3d bevel value */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	float GetBevel() const
	{
		return Bevel;
	}

	/** Set the 3d bevel value */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetBevel(const float Value);

	/** Get the 3d bevel type */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	EText3DBevelType GetBevelType() const
	{
		return BevelType;
	}

	/** Set the 3d bevel type */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetBevelType(const EText3DBevelType Value);

	/** Get the amount of segments that will be used to tessellate the Bevel */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	int32 GetBevelSegments() const
	{
		return BevelSegments;
	}

	/** Set the amount of segments that will be used to tessellate the Bevel */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetBevelSegments(const int32 Value);

	/** Get whether an outline is applied. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	bool GetUseOutline() const
	{
		return bUseOutline;
	}

	/** Set whether an outline is applied. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetUseOutline(const bool bValue);

	/** Get the outline width. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	float GetOutline() const
	{
		return Outline;
	}

	/** Set the outline width. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetOutline(const float Value);

	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetOutlineType(EText3DOutlineType InType);

	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	EText3DOutlineType GetOutlineType() const
	{
		return OutlineType;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetPivotHAlignment(EText3DHorizontalTextAlignment InPivot);

	EText3DHorizontalTextAlignment GetPivotHAlignment() const
	{
		return PivotHAlignment;
	}
	
	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	void SetPivotVAlignment(EText3DVerticalTextAlignment InPivot);

	EText3DVerticalTextAlignment GetPivotVAlignment() const
	{
		return PivotVAlignment;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Geometry")
	TEXT3D_API void SetCollisionEnabled(ECollisionEnabled::Type InCollisionEnabled);

	UFUNCTION(BlueprintPure, Category = "Text3D|Geometry")
	ECollisionEnabled::Type GetCollisionEnabled() const
	{
		return CollisionEnabled;
	}

protected:
	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DGeometryExtensionBase
	virtual EText3DHorizontalTextAlignment GetGlyphHAlignment() const override;
	virtual EText3DVerticalTextAlignment GetGlyphVAlignment() const override;
	virtual ECollisionEnabled::Type GetGlyphCollisionEnabled() const override;
	virtual const FGlyphMeshParameters* GetGlyphMeshParameters() const override;
	//~ End UText3DGeometryExtensionBase

	//~ Begin UText3DExtensionBase
	virtual void PrepareBuild(EText3DRendererFlags InUpdateFlags) override;
	virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	//~ End UText3DExtensionBase

	void OnGeometryOptionsChanged();
	void OnCollisionEnabledChanged();

	float GetMaxBevel() const;
	FVector GetPivotOffset() const;

	/** Size of the extrude */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (ClampMin = 0, AllowPrivateAccess = "true"))
	float Extrude = 5.0f;

	/** Size of bevel */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (EditCondition = "!bUseOutline && Extrude > 0", ClampMin = 0, AllowPrivateAccess = "true"))
	float Bevel = 0.0f;

	/** Bevel Type */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (EditCondition = "!bUseOutline && Extrude > 0", AllowPrivateAccess = "true"))
	EText3DBevelType BevelType = EText3DBevelType::Convex;

	/** Bevel Segments (Defines the amount of tesselation for the bevel part) */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (EditCondition = "!bUseOutline && Extrude > 0", ClampMin = 1, ClampMax = 15, AllowPrivateAccess = "true"))
	int32 BevelSegments = 8;

	/** Generate Outline */
	UPROPERTY(EditAnywhere, Getter = "GetUseOutline", Setter = "SetUseOutline", Category = "Geometry", meta = (AllowPrivateAccess = "true"))
	bool bUseOutline = false;

	/** Outline expand/offset amount, incompatible with bevel */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (EditCondition = "bUseOutline", AllowPrivateAccess = "true"))
	float Outline = 0.5f;

	/** Outline Type */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Geometry", meta = (EditCondition = "bUseOutline", AllowPrivateAccess = "true"))
	EText3DOutlineType OutlineType = EText3DOutlineType::Stroke;

	/** Pivot alignment of each character */
	UPROPERTY(EditAnywhere, DisplayName="Character Pivot Alignment Horizontal", Getter, Setter, Category = "Geometry", meta = (AllowPrivateAccess = "true"))
	EText3DHorizontalTextAlignment PivotHAlignment = EText3DHorizontalTextAlignment::Left;
	
	/** Pivot alignment of each character */
	UPROPERTY(Getter, Setter, meta=(ValidEnumValues="Bottom"))
	EText3DVerticalTextAlignment PivotVAlignment = EText3DVerticalTextAlignment::Bottom;

	UPROPERTY(EditAnywhere, Setter, Getter, Category="Geometry", meta=(AllowPrivateAccess = "true"))
	TEnumAsByte<ECollisionEnabled::Type> CollisionEnabled = ECollisionEnabled::QueryAndPhysics;

private:
	FGlyphMeshParameters GlyphMeshParameters;
};