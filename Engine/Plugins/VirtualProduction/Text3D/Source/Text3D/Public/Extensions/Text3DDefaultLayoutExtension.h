// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/Text3DLayoutExtensionBase.h"
#include "Fonts/FontCache.h"
#include "Text3DDefaultLayoutExtension.generated.h"

class FRichTextLayoutMarshaller;
class FText3DLayout;

namespace UE::Text3D::Layout
{
	struct FGlyphText;
}

UCLASS(MinimalAPI)
class UText3DDefaultLayoutExtension : public UText3DLayoutExtensionBase
{
	GENERATED_BODY()

public:
	static TEXT3D_API FName GetUseMaxWidthPropertyName();
	static TEXT3D_API FName GetUseMaxHeightPropertyName();
	static TEXT3D_API FName GetMaxHeightPropertyName();
	static TEXT3D_API FName GetMaxWidthPropertyName();
	static TEXT3D_API FName GetScaleProportionallyPropertyName();

	/** Get the tracking value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	float GetTracking() const
	{
		return Tracking;
	}

	/** Set the tracking value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetTracking(const float Value);

	/** Get the line spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	float GetLineSpacing() const
	{
		return LineSpacing;
	}

	/** Set the line spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetLineSpacing(const float Value);

	/** Get the word spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	float GetWordSpacing() const
	{
		return WordSpacing;
	}

	/** Set the word spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetWordSpacing(const float Value);

	/** Get the horizontal alignment value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	EText3DHorizontalTextAlignment GetHorizontalAlignment() const
	{
		return HorizontalAlignment;
	}

	/** Set the horizontal alignment value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetHorizontalAlignment(const EText3DHorizontalTextAlignment value);

	/** Get the vertical alignment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	EText3DVerticalTextAlignment GetVerticalAlignment() const
	{
		return VerticalAlignment;
	}

	/** Set the vertical alignment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetVerticalAlignment(const EText3DVerticalTextAlignment value);

	/** Whether a maximum width is specified */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	bool GetUseMaxWidth() const
	{
		return bUseMaxWidth;
	}

	/** Enable / Disable a Maximum Width */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetUseMaxWidth(const bool Value);

	/** Get the Maximum Width - If width is larger, mesh will scale down to fit MaxWidth value */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	float GetMaxWidth() const
	{
		return MaxWidth;
	}

	/** Set the Maximum Width - If width is larger, mesh will scale down to fit MaxWidth value */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetMaxWidth(const float Value);

	/** Get the Maximum Width Handling - Whether to wrap before scaling when the text size reaches the max width */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	EText3DMaxWidthHandling GetMaxWidthBehavior() const
	{
		return MaxWidthBehavior;
	}

	/** Set the Maximum Width Handling - Whether to wrap before scaling when the text size reaches the max width */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetMaxWidthBehavior(const EText3DMaxWidthHandling Value);

	/** Whether a maximum height is specified */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	bool GetUseMaxHeight() const
	{
		return bUseMaxHeight;
	}

	/** Enable / Disable a Maximum Height */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetUseMaxHeight(const bool Value);

	/** Get the Maximum Height - If height is larger, mesh will scale down to fit MaxHeight value */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	float GetMaxHeight() const
	{
		return MaxHeight;
	}

	/** Set the Maximum Height - If height is larger, mesh will scale down to fit MaxHeight value */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetMaxHeight(const float Value);

	/** Get if the mesh should scale proportionally when Max Width/Height is set */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	bool GetScalesProportionally() const
	{
		return bScaleProportionally;
	}

	/** Set if the mesh should scale proportionally when Max Width/Height is set */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetScaleProportionally(const bool Value);

	/** Get the current method of shaping text */
	UFUNCTION(BlueprintPure, Category = "Text3D|Layout")
	ETextShapingMethod GetTextShapingMethod() const
	{
		return TextShapingMethod;
	}

	/** Set the method to shape text */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Layout")
	void SetTextShapingMethod(ETextShapingMethod InTextShapingMethod);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DLayoutExtensionBase
	virtual float GetTextHeight() const override;
	virtual FVector GetTextScale() const override;
	virtual FVector GetLineLocation(int32 LineIndex) const override;
	//~ End UText3DLayoutExtensionBase

	//~ Begin UText3DExtensionBase
	virtual void PrepareBuild(EText3DRendererFlags InUpdateFlags) override;
	virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	//~ End UText3DExtensionBase

	void CalculateTextScale();
	void OnLayoutOptionsChanged();
	void OnTextShapingMethodChanged();

	/** Horizontal text alignment */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true", CustomAlignmentWidget))
	EText3DHorizontalTextAlignment HorizontalAlignment = EText3DHorizontalTextAlignment::Left;

	/** Vertical text alignment */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true", CustomAlignmentWidget))
	EText3DVerticalTextAlignment VerticalAlignment = EText3DVerticalTextAlignment::FirstLine;

	/** Text tracking affects all characters */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true"))
	float Tracking = 0.0f;

	/** Extra line spacing */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true"))
	float LineSpacing = 0.0f;

	/** Extra word spacing */
	UPROPERTY(EditAnywhere, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true"))
	float WordSpacing = 0.0f;

	/** Sets a maximum width to the 3D Text */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (EditCondition = "bUseMaxWidth", DisplayAfter="bUseMaxWidth", ClampMin = 1, AllowPrivateAccess = "true"))
	float MaxWidth = 500.f;

	/** Dictates how to handle the text if it exceeds the max width */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (EditCondition="bUseMaxWidth", DisplayAfter="MaxWidth", AllowPrivateAccess = "true"))
	EText3DMaxWidthHandling MaxWidthBehavior = EText3DMaxWidthHandling::Scale;

	/** Sets a maximum height to the 3D Text */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (EditCondition = "bUseMaxHeight", DisplayAfter="bUseMaxHeight", ClampMin = 1, AllowPrivateAccess = "true"))
	float MaxHeight = 500.0f;

	/** Enables a maximum width to the 3D Text */
	UPROPERTY(EditAnywhere, Getter = "GetUseMaxWidth", Setter = "SetUseMaxWidth", Category = "Layout", meta = (AllowPrivateAccess = "true"))
	bool bUseMaxWidth = false;

	/** Enables a maximum height to the 3D Text */
	UPROPERTY(EditAnywhere, Getter = "GetUseMaxHeight", Setter = "SetUseMaxHeight", Category = "Layout", meta = (AllowPrivateAccess = "true"))
	bool bUseMaxHeight = false;

	/** Should the mesh scale proportionally when Max Width/Height is set */
	UPROPERTY(EditAnywhere, Getter = "GetScalesProportionally", Setter = "SetScaleProportionally", Category = "Layout", meta = (AllowPrivateAccess = "true"))
	bool bScaleProportionally = true;

	/** The method to use for shaping the text */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Layout", meta = (AllowPrivateAccess = "true"), AdvancedDisplay)
	ETextShapingMethod TextShapingMethod = ETextShapingMethod::Auto;

private:
	/** Additional scale to apply to the text. */
	FVector TextScale = FVector::ZeroVector;

	/** Caches the last result of ShapedText, to allow faster updates of layout changes. */
	TSharedPtr<UE::Text3D::Layout::FGlyphText> GlyphText;

	/** Stores the text layout calculated by the TextLayoutMarshaller. */
	TSharedPtr<FText3DLayout> TextLayout;

	/** Determines how text is laid out, ie. parsing line breaks. */
	TSharedPtr<FRichTextLayoutMarshaller> TextLayoutMarshaller;

	/** Temporary storage for the glyph handles while text is built */
	TArray<UE::Text3D::Geometry::FCachedFontFaceGlyphHandle> GlyphHandles;
};