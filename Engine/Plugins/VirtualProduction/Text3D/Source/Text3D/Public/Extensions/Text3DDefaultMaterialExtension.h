// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DMaterialExtensionBase.h"
#include "Text3DDefaultMaterialExtension.generated.h"

class UMaterialInstanceDynamic;
class UTexture;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI)
class UText3DDefaultMaterialExtension : public UText3DMaterialExtensionBase
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetStyle(EText3DMaterialStyle InStyle);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	EText3DMaterialStyle GetStyle() const
	{
		return Style;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetFrontColor(const FLinearColor& InColor);
	
	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetFrontColor() const
	{
		return FrontColor;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetBackColor(const FLinearColor& InColor);
	
	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetBackColor() const
	{
		return BackColor;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetExtrudeColor(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetExtrudeColor() const
	{
		return ExtrudeColor;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetBevelColor(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetBevelColor() const
	{
		return BevelColor;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetGradientColorA(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetGradientColorA() const
	{
		return GradientColorA;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetGradientColorB(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FLinearColor& GetGradientColorB() const
	{
		return GradientColorB;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetGradientSmoothness(float InGradientSmoothness);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetGradientSmoothness() const
	{
		return GradientSmoothness;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetGradientOffset(float InGradientOffset);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetGradientOffset() const
	{
		return GradientOffset;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetGradientRotation(float InGradientRotation);
	
	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetGradientRotation() const
	{
		return GradientRotation;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetTextureAsset(UTexture2D* InTextureAsset);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	UTexture2D* GetTextureAsset() const
	{
		return TextureAsset;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetTextureTiling(const FVector2D& InTextureTiling);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	const FVector2D& GetTextureTiling() const
	{
		return TextureTiling;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetBlendMode(EText3DMaterialBlendMode InBlendMode);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	EText3DMaterialBlendMode GetBlendMode() const
	{
		return BlendMode;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetIsUnlit(bool bInIsUnlit);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	bool GetIsUnlit() const
	{
		return bIsUnlit;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetOpacity(float InOpacity);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetOpacity() const
	{
		return Opacity;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetUseMask(bool bInUseMask);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	bool GetUseMask() const
	{
		return bUseMask;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetMaskOffset(float InMaskOffset);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetMaskOffset() const
	{
		return MaskOffset;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetMaskSmoothness(float InMaskSmoothness);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetMaskSmoothness() const
	{
		return MaskSmoothness;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetMaskRotation(float InMaskRotation);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	float GetMaskRotation() const
	{
		return MaskRotation;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetUseSingleMaterial(bool bInUse);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	bool GetUseSingleMaterial() const
	{
		return bUseSingleMaterial;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetFrontMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	UMaterialInterface* GetFrontMaterial() const
	{
		return FrontMaterial;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetBevelMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	UMaterialInterface* GetBevelMaterial() const
	{
		return BevelMaterial;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetExtrudeMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	UMaterialInterface* GetExtrudeMaterial() const
	{
		return ExtrudeMaterial;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Material")
	TEXT3D_API void SetBackMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintPure, Category = "Text3D|Material")
	UMaterialInterface* GetBackMaterial() const
	{
		return BackMaterial;
	}

	TEXT3D_API FVector GetGradientDirection() const;

	TEXT3D_API void PreCacheMaterials();

	//~ Begin UObject
	TEXT3D_API virtual void PostLoad() override;
	TEXT3D_API virtual void PostInitProperties() override;
	TEXT3D_API virtual void PostDuplicate(bool bInDuplicateForPIE) override;
#if WITH_EDITOR
	TEXT3D_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
	TEXT3D_API virtual void PostEditUndo() override;
#endif
	//~ End UObject

	//~ Begin UText3DExtensionBase
	TEXT3D_API virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	TEXT3D_API virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	//~ End UText3DExtensionBase

	//~ Begin UText3DMaterialExtensionBase
	TEXT3D_API virtual void SetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial) override;
	TEXT3D_API virtual UMaterialInterface* GetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters) const override;
	TEXT3D_API virtual void RegisterMaterialOverride(FName InTag) override;
	TEXT3D_API virtual void UnregisterMaterialOverride(FName InTag) override;
	TEXT3D_API virtual void ForEachMaterial(TFunctionRef<bool(const UE::Text3D::Material::FMaterialParameters&, UMaterialInterface*)> InFunctor) const override;
	TEXT3D_API virtual int32 GetMaterialCount() const override;
	TEXT3D_API virtual void GetMaterialNames(TArray<FName>& OutNames) const override;
	//~ End UText3DMaterialExtensionBase

protected:
	UMaterialInstanceDynamic* FindOrAdd(EText3DGroupType InGroup, FName InTag = NAME_None);
	UMaterialInstanceDynamic* Find(EText3DGroupType InGroup, FName InTag = NAME_None) const;
	void SetVectorParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, FVector InValue);
	void SetVectorParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, FLinearColor InValue);
	void SetScalarParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, float InValue);
	void SetTextureParameter(TConstArrayView<UMaterialInstanceDynamic*> InMaterials, FName InKey, UTexture* InValue);
	void OnMaterialOptionsChanged();
	void OnCustomMaterialChanged();

	/** Clears the parameter values of all materials that are MIDs in this extension */
	void ClearParameterValues();

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(AllowPrivateAccess="true"))
	EText3DMaterialStyle Style = EText3DMaterialStyle::Solid;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Solid", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor FrontColor = FLinearColor::White;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Solid", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor BackColor = FLinearColor::White;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Solid", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor ExtrudeColor = FLinearColor::Gray;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Solid", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor BevelColor = FLinearColor::Gray;
	
	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Gradient", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor GradientColorA = FLinearColor::White;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Gradient", EditConditionHides, AllowPrivateAccess="true"))
	FLinearColor GradientColorB = FLinearColor::Black;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(ClampMin=0.0, ClampMax=1.0, EditCondition="Style == EText3DMaterialStyle::Gradient", EditConditionHides, AllowPrivateAccess="true"))
	float GradientSmoothness = 0.1f;
	
	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta = (ClampMin=0.0, ClampMax=1.0, EditCondition="Style == EText3DMaterialStyle::Gradient", EditConditionHides, AllowPrivateAccess="true"))
	float GradientOffset = 0.5f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta = (ClampMin=0.0, ClampMax=1.0, EditCondition="Style == EText3DMaterialStyle::Gradient", EditConditionHides, AllowPrivateAccess="true"))
	float GradientRotation = 0.0f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style == EText3DMaterialStyle::Texture", EditConditionHides, AllowPrivateAccess="true"))
	TObjectPtr<UTexture2D> TextureAsset;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(Delta=0.1, EditCondition="Style == EText3DMaterialStyle::Texture", EditConditionHides, AllowPrivateAccess="true"))
	FVector2D TextureTiling = FVector2D::One();

	UPROPERTY(EditAnywhere, Getter="GetIsUnlit", Setter="SetIsUnlit", Category="Material", meta=(EditCondition="Style != EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess="true"))
	bool bIsUnlit = true;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(EditCondition="Style != EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess="true"))
	EText3DMaterialBlendMode BlendMode = EText3DMaterialBlendMode::Opaque;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(ClampMin=0.0, ClampMax=1.0, EditCondition="Style != EText3DMaterialStyle::Custom && BlendMode == EText3DMaterialBlendMode::Translucent", EditConditionHides, AllowPrivateAccess="true"))
	float Opacity = 1.f;

	/** Enable text shader mask */
	UPROPERTY(EditAnywhere, Getter="GetUseMask", Setter="SetUseMask", Category="Material", meta=(EditCondition="Style != EText3DMaterialStyle::Custom && BlendMode == EText3DMaterialBlendMode::Translucent", EditConditionHides, AllowPrivateAccess="true"))
	bool bUseMask = false;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(ClampMin=0.0, ClampMax=1.0, EditCondition="bUseMask && Style != EText3DMaterialStyle::Custom && BlendMode == EText3DMaterialBlendMode::Translucent", EditConditionHides, AllowPrivateAccess="true"))
	float MaskOffset = 1.f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(ClampMin=0.0, ClampMax=1.0, EditCondition="bUseMask && Style != EText3DMaterialStyle::Custom && BlendMode == EText3DMaterialBlendMode::Translucent", EditConditionHides, AllowPrivateAccess="true"))
	float MaskSmoothness = 0.1f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category="Material", meta=(ClampMin=0.0, ClampMax=1.0, EditCondition="bUseMask && Style != EText3DMaterialStyle::Custom && BlendMode == EText3DMaterialBlendMode::Translucent", EditConditionHides, AllowPrivateAccess="true"))
	float MaskRotation = 0.0f;

	/** Use primary material for all available slots */
	UPROPERTY(EditAnywhere, Getter="GetUseSingleMaterial", Setter="SetUseSingleMaterial", Category = "Material", meta = (EditCondition="Style == EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess = "true"))
	bool bUseSingleMaterial = false;

	/** Material for the front part */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Material", meta = (EditCondition="Style == EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> FrontMaterial;

	/** Material for the bevel part */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Material", meta = (EditCondition="!bUseSingleMaterial && Style == EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> BevelMaterial;

	/** Material for the extruded part */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Material", meta = (EditCondition="!bUseSingleMaterial && Style == EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> ExtrudeMaterial;

	/** Material for the back part */
	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Material", meta = (EditCondition="!bUseSingleMaterial && Style == EText3DMaterialStyle::Custom", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> BackMaterial;

	/** Additional overrides defined for specific group */
	UPROPERTY(Transient, DuplicateTransient)
	TArray<FText3DMaterialOverride> MaterialOverrides;

	/** Cached group dynamic materials created during session based on their options */
	UPROPERTY(Transient, DuplicateTransient)
	TMap<FText3DMaterialGroupKey, TObjectPtr<UMaterialInstanceDynamic>> GroupDynamicMaterials;

	/** Used to pass bounds to material */
	FBox LocalTextBounds = FBox(ForceInitToZero);
};
