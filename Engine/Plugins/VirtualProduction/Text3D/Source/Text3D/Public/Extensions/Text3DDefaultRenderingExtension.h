// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DRenderingExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DDefaultRenderingExtension.generated.h"

/** Extension that handles rendering data for Text3D */
UCLASS(MinimalAPI)
class UText3DDefaultRenderingExtension : public UText3DRenderingExtensionBase
{
	GENERATED_BODY()

public:
	/** Get the value of CastShadow. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Rendering")
	bool GetCastShadow() const
	{
		return bCastShadow;
	}

	/** Set the value of CastShadow. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Rendering")
	void SetCastShadow(bool bInCastShadow);
	
	/** Get the value of CastHiddenShadow. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Rendering")
	bool GetCastHiddenShadow() const
	{
		return bCastHiddenShadow;
	}

	/** Set the value of CastHiddenShadow. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Rendering")
	void SetCastHiddenShadow(bool bInCastShadow);
	
	/** Get the value of AffectDynamicIndirectLighting. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Rendering")
	bool GetAffectDynamicIndirectLighting() const
	{
		return bAffectDynamicIndirectLighting;
	}

	/** Set the value of AffectDynamicIndirectLighting. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Rendering")
	void SetAffectDynamicIndirectLighting(bool bInValue);

	/** Get the value of AffectIndirectLightingWhileHidden. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Rendering")
	bool GetAffectIndirectLightingWhileHidden() const
	{
		return bAffectIndirectLightingWhileHidden;
	}

	/** Set the value of AffectIndirectLightingWhileHidden. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Rendering")
	void SetAffectIndirectLightingWhileHidden(bool bInValue);

	/** Get the value of Holdout. */
	UFUNCTION(BlueprintPure, Category = "Text3D|Rendering")
	bool GetHoldout() const
	{
		return bHoldout;
	}

	/** Set the value of Holdout. */
	UFUNCTION(BlueprintCallable, Category = "Text3D|Rendering")
	void SetHoldout(bool bInHoldout);
	
protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DRenderingExtensionBase
	virtual bool GetTextCastShadow() const override;
	virtual bool GetTextCastHiddenShadow() const override;
	virtual bool GetTextAffectDynamicIndirectLighting() const override;
	virtual bool GetTextAffectIndirectLightingWhileHidden() const override;
	virtual bool GetTextHoldout() const override;
	//~ End UText3DRenderingExtensionBase

	void OnRenderingOptionsChanged();

	/** Controls whether the text glyphs should cast a shadow or not. */
	UPROPERTY(EditAnywhere, Getter = "GetCastShadow", Setter = "SetCastShadow", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bCastShadow = true;

	/** Controls whether the text glyphs should cast a shadow or not. */
	UPROPERTY(EditAnywhere, DisplayName = "Hidden Shadow", Getter = "GetCastShadow", Setter = "SetCastShadow", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bCastHiddenShadow = false;

	/** Controls whether the primitive should influence indirect lighting. */
	UPROPERTY(EditAnywhere, Getter = "GetAffectDynamicIndirectLighting", Setter = "SetAffectDynamicIndirectLighting", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bAffectDynamicIndirectLighting = true;

	/** Controls whether the primitive should affect indirect lighting when hidden. This flag is only used if bAffectDynamicIndirectLighting is true. */
	UPROPERTY(EditAnywhere, Getter="GetAffectIndirectLightingWhileHidden", Setter="SetAffectIndirectLightingWhileHidden", Category = "Rendering", meta = (EditCondition = "bAffectDynamicIndirectLighting", AllowPrivateAccess = "true"))
	bool bAffectIndirectLightingWhileHidden = false;
	
	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer. */
	UPROPERTY(EditAnywhere, Getter="GetHoldout", Setter="SetHoldout", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bHoldout = false;
};
