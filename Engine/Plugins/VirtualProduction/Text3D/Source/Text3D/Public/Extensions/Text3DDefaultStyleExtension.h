// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Text3DStyleExtensionBase.h"
#include "Text3DDefaultStyleExtension.generated.h"

class FSlateStyleSet;
class UText3DStyleBase;
class UText3DStyleSet;
struct FTextBlockStyle;

UCLASS(MinimalAPI)
class UText3DDefaultStyleExtension : public UText3DStyleExtensionBase
{
	GENERATED_BODY()

public:
	UText3DDefaultStyleExtension();

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	void SetStyleSet(UText3DStyleSet* InStyleSet);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	UText3DStyleSet* GetStyleSet() const
	{
		return StyleSet;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DStyleExtensionBase
	virtual TSharedPtr<FTextBlockStyle> GetDefaultStyle() const override;
    virtual TSharedPtr<FSlateStyleSet> GetCustomStyles() const override;
	virtual UText3DStyleBase* GetStyle(FName InStyle) const override;
    //~ End UText3DStyleExtensionBase

    //~ Begin UText3DExtensionBase
	virtual void PrepareBuild(EText3DRendererFlags InUpdateFlags) override;
    virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
    virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
    //~ End UText3DExtensionBase

	void OnStyleSetUpdated(UText3DStyleSet* InStyleSet);
	void OnStyleSetChanged();

	/** A style set asset that defines custom style rules */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DStyleSet> StyleSet;

	/**
	 * Define your styles and use them in text like this <MyStyle>...</>,
	 * the text between start and end tag will have this style applied
	 */
	UPROPERTY(EditAnywhere, Instanced, DisplayName="Styles Override", Category="Style", NoClear, meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UText3DStyleBase>> Styles;

private:
	TSharedPtr<FTextBlockStyle> DefaultFontStyle;
    TSharedPtr<FSlateStyleSet> TextStyleSet;
};
