// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Text3DTokenExtensionBase.h"
#include "Text3DDefaultTokenExtension.generated.h"

class UText3DTokenBase;

UCLASS(MinimalAPI)
class UText3DDefaultTokenExtension : public UText3DTokenExtensionBase
{
	GENERATED_BODY()

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DTokenExtensionBase
	virtual const FText& GetFormattedText() const override;
	//~ End UText3DTokenExtensionBase

	//~ Begin UText3DExtensionBase
	virtual void PrepareBuild(EText3DRendererFlags InUpdateFlags) override;
	virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	//~ End UText3DExtensionBase

	/**
	 * Define your tokens and use them in text like this {MyCustomToken},
	 * {MyCustomToken} will be swapped by the content of the token called MyCustomToken,
	 * This can be used with remote control or sequencer to swap part of the text
	 */
	UPROPERTY(EditAnywhere, Instanced, Category="Token", NoClear, meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UText3DTokenBase>> Tokens;

	UPROPERTY(Transient)
	FText FormattedText;
};
