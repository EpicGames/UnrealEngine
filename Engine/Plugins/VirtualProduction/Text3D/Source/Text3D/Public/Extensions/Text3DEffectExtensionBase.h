// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DEffectExtensionBase.generated.h"

struct FText3DRange
{
	static constexpr uint16 Unbound = std::numeric_limits<uint16>::max();

	FText3DRange() = default;
	explicit FText3DRange(uint16 InStartIndex, uint16 InCount)
		: StartIndex(InStartIndex)
		, Count(InCount)
	{
	}

	bool IsUnbound() const
	{
		return IsStartUnbound() && IsEndUnbound();
	}

	bool IsEndUnbound() const
	{
		return Count == Unbound;
	}

	bool IsStartUnbound() const
	{
		return StartIndex == Unbound;
	}

	bool IsInRange(uint16 InIndex) const
	{
		return IsUnbound()
			|| (IsStartUnbound() && InIndex < Count)
			|| (IsEndUnbound() && InIndex >= StartIndex)
			|| (InIndex >= StartIndex && InIndex <= (StartIndex + Count));
	}

	uint16 StartIndex = Unbound;
	uint16 Count = Unbound;
};

/** Extension for custom effects on Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DEffectExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DEffectExtensionBase()
		: UText3DExtensionBase()
	{}

	explicit UText3DEffectExtensionBase(uint16 InPriority)
		: UText3DExtensionBase(InPriority)
	{}

protected:
	//~ Begin UText3DExtensionBase
	TEXT3D_API virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	TEXT3D_API virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) override;
	//~ End UText3DExtensionBase

	virtual FText3DRange GetTargetRange() const
	{
		return FText3DRange();
	}

	virtual void ApplyEffect(uint32 InCharacterIndex, uint32 InCharacterCount)
	{
	}
};