// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DTypes.h"
#include "UObject/Object.h"
#include "Text3DExtensionBase.generated.h"

class UText3DComponent;

enum class EText3DExtensionResult : uint8
{
	/** Extension is still needed, keep it around */
	Active,
	/** Extension is done for this update round, should not be called anymore */
	Finished,
	/** Extension failed to execute properly, do not continue */
	Failed,
};

/**
 * Extensions are piece of data and logic needed to render Text3D that can be reused by multiple renderers,
 * they execute once during every renderer update at the right moment
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UText3DExtensionBase : public UObject
{
	GENERATED_BODY()

public:
	UText3DExtensionBase()
		: UText3DExtensionBase(0)
	{}

	UText3DExtensionBase(uint16 InPriority)
		: UpdatePriority(InPriority)
	{}

	/** Prepare the extensions before a build occurs */
	virtual void PrepareBuild(EText3DRendererFlags InUpdateFlags)
	{
	}

	/** Perform an update of the extension behavior before renderer execute based on condition above */
	virtual EText3DExtensionResult PreRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
	{
		return EText3DExtensionResult::Finished;
	}

	/** Perform an update of the extension behavior after renderer execute based on condition above */
	virtual EText3DExtensionResult PostRendererUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters)
	{
		return EText3DExtensionResult::Finished;
	}

	/** Used to sort extensions and execute them before others */
	uint16 GetUpdatePriority() const
	{
		return UpdatePriority;
	}

	TEXT3D_API UText3DComponent* GetText3DComponent() const;

protected:
	void RequestUpdate(EText3DRendererFlags InFlags, bool bInImmediateUpdate=false) const;

private:
	uint16 UpdatePriority = 0;
};
