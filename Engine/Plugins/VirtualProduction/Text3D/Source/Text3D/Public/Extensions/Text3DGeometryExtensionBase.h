// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DGeometryExtensionBase.generated.h"

class FFreeTypeFace;
class UStaticMesh;
class UText3DCharacterBase;
struct FText3DCachedMesh;
struct FTypefaceFontData;

/** Extension that handles geometry data for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DGeometryExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DGeometryExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Geometry)
	{}

	// Deprecated do not use anymore, use character getter directly
	UE_DEPRECATED(5.7, "Deprecated do not use anymore, use character getter directly")
	virtual const FText3DCachedMesh* FindOrLoadGlyphMesh(UText3DCharacterBase* InCharacter) const final
	{
		return nullptr;
	}

	virtual EText3DHorizontalTextAlignment GetGlyphHAlignment() const
	{
		return EText3DHorizontalTextAlignment::Left;
	}

	virtual EText3DVerticalTextAlignment GetGlyphVAlignment() const
	{
		return EText3DVerticalTextAlignment::Bottom;
	}

	virtual ECollisionEnabled::Type GetGlyphCollisionEnabled() const
	{
		return ECollisionEnabled::QueryAndPhysics;
	}

	virtual const FGlyphMeshParameters* GetGlyphMeshParameters() const
	{
		return nullptr;
	}
};
