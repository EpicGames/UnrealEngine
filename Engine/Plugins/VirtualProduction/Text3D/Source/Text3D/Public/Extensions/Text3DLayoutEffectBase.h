// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DEffectExtensionBase.h"
#include "Text3DLayoutEffectBase.generated.h"

/** Extension for layout effects on Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DLayoutEffectBase : public UText3DEffectExtensionBase
{
	GENERATED_BODY()

public:
	UText3DLayoutEffectBase()
		: UText3DEffectExtensionBase(UE::Text3D::Priority::Effect)
	{}
};
