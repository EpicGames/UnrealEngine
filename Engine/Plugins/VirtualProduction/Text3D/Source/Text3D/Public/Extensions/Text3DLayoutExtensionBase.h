// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DLayoutExtensionBase.generated.h"

struct FText3DShapedGlyphText;

/** Extension that handles layout data for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DLayoutExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DLayoutExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Layout)
	{}

	virtual float GetTextHeight() const
	{
		return 0.f;
	}

	virtual FVector GetTextScale() const
	{
		return FVector::OneVector;
	}

	virtual FVector GetLineLocation(int32 LineIndex) const
	{
		return FVector::ZeroVector;
	}
};
