// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DLayoutEffectBase.h"
#include "Text3DTypes.h"
#include "Text3DLayoutTransformEffect.generated.h"

class UCurveFloat;

/** Extension that handles transform data for Text3D */
UCLASS(MinimalAPI, EditInlineNew, ClassGroup=Text3D, DisplayName="TransformEffect", AutoExpandCategories=(Transform))
class UText3DLayoutTransformEffect : public UText3DLayoutEffectBase
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationEnabled(bool bEnabled);

	bool GetLocationEnabled() const
	{
		return bLocationEnabled;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationProgress(float Progress);

	float GetLocationProgress() const
	{
		return LocationProgress;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationOrder(EText3DCharacterEffectOrder Order);

	EText3DCharacterEffectOrder GetLocationOrder() const
	{
		return LocationOrder;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationBegin(const FVector& InBegin);

	FVector GetLocationBegin() const
	{
		return LocationBegin;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationEnd(const FVector& InEnd);

	FVector GetLocationEnd() const
	{
		return LocationEnd;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationEaseCurve(UCurveFloat* InEaseCurve);

	UCurveFloat* GetLocationEaseCurve() const
	{
		return LocationEaseCurve;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetLocationStagger(bool bInStagger);

	bool GetLocationStagger() const
	{
		return bLocationStagger;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationEnabled(bool bEnabled);

	bool GetRotationEnabled() const
	{
		return bRotationEnabled;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationProgress(float Progress);

	float GetRotationProgress() const
	{
		return RotationProgress;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationOrder(EText3DCharacterEffectOrder Order);

	EText3DCharacterEffectOrder GetRotationOrder() const
	{
		return RotationOrder;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationBegin(const FRotator& Value);

	FRotator GetRotationBegin() const
	{
		return RotationBegin;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationEnd(const FRotator& Value);

	FRotator GetRotationEnd() const
	{
		return RotationEnd;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationEaseCurve(UCurveFloat* InEaseCurve);

	UCurveFloat* GetRotationEaseCurve() const
	{
		return RotationEaseCurve;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetRotationStagger(bool bInStagger);

	bool GetRotationStagger() const
	{
		return bRotationStagger;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleEnabled(bool bEnabled);

	bool GetScaleEnabled() const
	{
		return bScaleEnabled;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleProgress(float Progress);

	float GetScaleProgress() const
	{
		return ScaleProgress;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleOrder(EText3DCharacterEffectOrder Order);

	EText3DCharacterEffectOrder GetScaleOrder() const
	{
		return ScaleOrder;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleBegin(const FVector& Value);

	FVector GetScaleBegin() const
	{
		return ScaleBegin;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleEnd(const FVector& Value);

	FVector GetScaleEnd() const
	{
		return ScaleEnd;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleEaseCurve(UCurveFloat* InEaseCurve);

	UCurveFloat* GetScaleEaseCurve() const
	{
		return ScaleEaseCurve;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Transform")
	TEXT3D_API void SetScaleStagger(bool bInStagger);

	bool GetScaleStagger() const
	{
		return bScaleStagger;
	}

protected:
	UText3DLayoutTransformEffect();

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UText3DTransformExtensionBase
	virtual void ApplyEffect(uint32 InGlyphIndex, uint32 InGlyphCount) override;
	//~ End UText3DTransformExtensionBase

	void OnTransformOptionsChanged();

	int32 GetEffectPosition(int32 Index, int32 Total, EText3DCharacterEffectOrder Order) const;
	float CalculateEffect(int32 InIndex, int32 InTotal, EText3DCharacterEffectOrder InOrder, float InProgress, const UCurveFloat* InEaseCurve, bool bInStagger) const;

	// Location

	UPROPERTY(EditAnywhere, Getter="GetLocationEnabled", Setter="SetLocationEnabled", Category = "Transform")
	bool bLocationEnabled = false;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides, ClampMin = 0, ClampMax = 100, Units=Percent))
	float LocationProgress = 0.f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides))
	EText3DCharacterEffectOrder LocationOrder = EText3DCharacterEffectOrder::Normal;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides))
	FVector LocationBegin = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides))
	FVector LocationEnd = FVector(100.f, 0.f, 0.f);

	/** Provide a 0-1 ease curve, leaving this unset will result in linear ease */
	UPROPERTY(EditAnywhere, Setter, Getter, Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides))
	TObjectPtr<UCurveFloat> LocationEaseCurve;

	UPROPERTY(EditAnywhere, Getter="GetLocationStagger", Setter="SetLocationStagger", Category = "Transform", meta = (EditCondition = "bLocationEnabled", EditConditionHides))
	bool bLocationStagger = false; 

	// Rotate

	UPROPERTY(EditAnywhere, Getter="GetRotationEnabled", Setter="SetRotationEnabled", Category = "Transform")
	bool bRotationEnabled = false;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides, ClampMin = 0, ClampMax = 100, Units=Percent))
	float RotationProgress = 0.0f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides))
	EText3DCharacterEffectOrder RotationOrder = EText3DCharacterEffectOrder::Normal;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides))
	FRotator RotationBegin = FRotator(-90.f, 0.f, 0.f);

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides))
	FRotator RotationEnd = FRotator(0.f, 0.f, 0.f);

	/** Provide a 0-1 ease curve, leaving this unset will result in linear ease */
	UPROPERTY(EditAnywhere, Setter, Getter, Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides))
	TObjectPtr<UCurveFloat> RotationEaseCurve;

	UPROPERTY(EditAnywhere, Getter="GetRotationStagger", Setter="SetRotationStagger", Category = "Transform", meta = (EditCondition = "bRotationEnabled", EditConditionHides))
	bool bRotationStagger = false;

	// Scale

	UPROPERTY(EditAnywhere, Getter="GetScaleEnabled", Setter="SetScaleEnabled", Category = "Transform")
	bool bScaleEnabled = false;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides, ClampMin = 0, ClampMax = 100, Units=Percent))
	float ScaleProgress = 0.f;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides))
	EText3DCharacterEffectOrder ScaleOrder = EText3DCharacterEffectOrder::Normal;

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides, ClampMin = 0))
	FVector ScaleBegin = FVector(1.f, 0.f, 0.f);

	UPROPERTY(EditAnywhere, Getter, Setter, Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides, ClampMin = 0))
	FVector ScaleEnd = FVector::OneVector;

	/** Provide a 0-1 ease curve, leaving this unset will result in linear ease */
	UPROPERTY(EditAnywhere, Setter, Getter, Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides))
	TObjectPtr<UCurveFloat> ScaleEaseCurve;

	UPROPERTY(EditAnywhere, Getter="GetScaleStagger", Setter="SetScaleStagger", Category = "Transform", meta = (EditCondition = "bScaleEnabled", EditConditionHides))
	bool bScaleStagger = false;
};
