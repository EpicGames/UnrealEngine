// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DMaterialExtensionBase.generated.h"

class UMaterialInterface;

/** Extension that handles materials for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DMaterialExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DMaterialExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Material)
	{}

	/** Set default material for a specific group */
	UE_DEPRECATED(5.7, "Please use SetMaterial with FMaterialParameters instead")
	virtual void SetMaterial(EText3DGroupType InGroup, UMaterialInterface* InMaterial)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = InGroup;
		SetMaterial(Parameters, InMaterial);
	}

	/** Get default material of a specific group */
	UE_DEPRECATED(5.7, "Please use GetMaterial with FMaterialParameters instead")
	virtual UMaterialInterface* GetMaterial(EText3DGroupType InGroup) const
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Group = InGroup;
		return GetMaterial(Parameters);
	}

	/** Set the material for a specific group */
	virtual void SetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
	{
		
	}

	/** Get the material for a specific group */
	virtual UMaterialInterface* GetMaterial(const UE::Text3D::Material::FMaterialParameters& InParameters) const
	{
		return nullptr;
	}

	/** Register a new material override */
	virtual void RegisterMaterialOverride(FName InTag)
	{
	}

	/** Unregister a material override previously defined */
	virtual void UnregisterMaterialOverride(FName InTag)
	{
	}

	/** Loop through each material available, return false to stop the operation */
	virtual void ForEachMaterial(TFunctionRef<bool(const UE::Text3D::Material::FMaterialParameters&, UMaterialInterface*)> InFunctor) const
	{
	}

	/** Get amount of material in use */
	virtual int32 GetMaterialCount() const
	{
		return 0;
	}

	/** Retrieve material override names */
	virtual void GetMaterialNames(TArray<FName>& OutNames) const
	{
	}
};
