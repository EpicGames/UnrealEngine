// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DRenderingExtensionBase.generated.h"

/** Extension that handles rendering data for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DRenderingExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DRenderingExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Rendering)
	{}

	virtual bool GetTextCastShadow() const
	{
		return true;
	}

	virtual bool GetTextCastHiddenShadow() const
	{
		return false;
	}

	virtual bool GetTextAffectDynamicIndirectLighting() const
	{
		return false;
	}

	virtual bool GetTextAffectIndirectLightingWhileHidden() const
	{
		return false;
	}
	
	virtual bool GetTextHoldout() const
	{
		return false;
	}
};
