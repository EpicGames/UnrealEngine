// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Templates/SharedPointer.h"
#include "Text3DStyleExtensionBase.generated.h"

class FSlateStyleSet;
class UText3DStyleBase;
struct FText3DShapedGlyphText;
struct FTextBlockStyle;

/** Extension that handles formatting styles for Text3D */
UCLASS(MinimalAPI, Abstract)
class UText3DStyleExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DStyleExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Style)
	{}

	/** Default style used by text when no other style is applied */
	virtual TSharedPtr<FTextBlockStyle> GetDefaultStyle() const
	{
		return nullptr;
	}

	/** Custom defined styles to apply on text */
	virtual TSharedPtr<FSlateStyleSet> GetCustomStyles() const
	{
		return nullptr;
	}

	/** Get the style with a specific name if any */
	virtual UText3DStyleBase* GetStyle(FName InStyleName) const
	{
		return nullptr;
	}
};
