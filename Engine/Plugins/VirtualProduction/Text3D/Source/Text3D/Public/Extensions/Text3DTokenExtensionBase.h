// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DExtensionBase.h"
#include "Text3DTypes.h"
#include "Text3DTokenExtensionBase.generated.h"

/** Extension that handles text transformation */
UCLASS(MinimalAPI, Abstract)
class UText3DTokenExtensionBase : public UText3DExtensionBase
{
	GENERATED_BODY()

public:
	UText3DTokenExtensionBase()
		: UText3DExtensionBase(UE::Text3D::Priority::Token)
	{}

	/** Get the transformed text */
	virtual const FText& GetFormattedText() const
	{
		return FText::GetEmpty();
	}
};
