// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IText3DBuildSystem.generated.h"

class FSceneView;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UText3DBuildSystemInterface : public UInterface
{
	GENERATED_BODY()
};

class IText3DBuildSystemInterface
{
	GENERATED_BODY()

public:
	/** Returns true if there are builders yet to complete */
	virtual bool IsBuildInProgress() const = 0;

	/**
	 * Executes any remaining text builds with no time budget. 
	 * Warning: This can cause a hitch and should only be used when the result is needed immediately regardless of hitch. 
	 */
	virtual void FlushBuilds() = 0;

	/** Adds the Text3D-managed primitives that are being built to the scene view's hidden primitive list */
	virtual void HidePrimitivesBeingBuilt(FSceneView& InSceneView) const = 0;
};
