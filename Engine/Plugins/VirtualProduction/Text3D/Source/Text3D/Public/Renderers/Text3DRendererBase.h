// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Text3DTypes.h"
#include "UObject/Object.h"
#include "Text3DRendererBase.generated.h"

class UText3DComponent;
class UPrimitiveComponent;

/**
 * Base class for a rendering implementation of Text3D
 * The whole rendering logic should be encapsulated into an instance of this class
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UText3DRendererBase : public UObject
{
	GENERATED_BODY()

public:
	/** Allocate renderer resources */
	void Create();

	/** Update rendering state */
	void Update(const UE::Text3D::Renderer::FUpdateParameters& InParameters);

	/** Clears the active rendering state */
	void Clear();

	/** Cleanup renderer resources */
	void Destroy();

	/** Get cached bounds from last update */
	FBox GetBounds() const;

	/** Calls the given functor for every primitive the renderer implementation manages */
	void IterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const;

	/** Get the mesh type that this renderer uses */
	virtual EText3DMeshType GetMeshType() const PURE_VIRTUAL(UText3DRendererBase::GetMeshType, return EText3DMeshType::Unknown; );

	/** Get the implementation name for debug purposes */
	virtual FName GetFriendlyName() const PURE_VIRTUAL(UText3DRendererBase::GetFriendlyName, return NAME_None; );

	/** Is the renderer initialized (render state created) */
	bool IsInitialized() const
	{
		return bInitialized;
	}

protected:
	/** Create and setup the implementation components, called on load or creation */
	virtual void OnCreate() PURE_VIRTUAL(UText3DRendererBase::OnCreate, );

	/** Update rendering state of text characters, called when render state is outdated */
	virtual void OnUpdate(const UE::Text3D::Renderer::FUpdateParameters& InParameters) PURE_VIRTUAL(UText3DRendererBase::OnUpdate, );

	/** Clear rendering state and remove all visible characters */
	virtual void OnClear() PURE_VIRTUAL(UText3DRendererBase::OnClear, );

	/** Destroy and clean the implementation components, called on destroy or deactivation */
	virtual void OnDestroy() PURE_VIRTUAL(UText3DRendererBase::OnDestroy, );

	/** Calculate the bounds of the rendered text */
	virtual FBox OnCalculateBounds() const PURE_VIRTUAL(UText3DRendererBase::OnCalculateBounds, return FBox(ForceInitToZero); );

	/** Calls the given functor for every primitive the renderer implementation manages */
	virtual void OnIterateManagedPrimitives(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const
	{
	}

#if WITH_EDITOR
	void BindDebugDelegate();
	void UnbindDebugDelegate();
	void OnTextSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent);
	void OnDebugModeChanged();

	/** Called when entering debug mode in editor */
	virtual void OnDebugModeEnabled() {}

	/** Called when exiting debug mode in editor */
	virtual void OnDebugModeDisabled() {}
#endif

	UText3DComponent* GetText3DComponent() const;

	/** Recalculates bounds when a layout or geometry change happened */
	void RefreshBounds();

private:
	TOptional<FBox> CachedBounds;
	bool bInitialized = false;

	friend struct FText3DComponentInstanceData;
};