// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Text3DTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "Text3DProjectSettings.generated.h"

enum class EText3DMaterialBlendMode : uint8;
class UFontFace;
class UFont;
class UMaterial;
class UText3DStyleSet;

/** Settings for Text3D plugin */
UCLASS(MinimalAPI, Config=Text3D, meta=(DisplayName="Text3D"))
class UText3DProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	friend class UText3DEditorFontSubsystem;

public:
	static TEXT3D_API const UText3DProjectSettings* Get();
	static TEXT3D_API UText3DProjectSettings* GetMutable();

#if WITH_EDITOR
	static FName GetDebugModePropertyName();
	static FName GetDefaultMaterialPropertyName();
#endif

	struct FMaterialParameters
	{
		static const FLazyName Mode;

		// Solid
		static const FLazyName SolidColor;
		static const FLazyName Opacity;
		
		// Gradient
		static const FLazyName GradientOffset;
		static const FLazyName GradientColorA;
		static const FLazyName GradientColorB;
		static const FLazyName GradientRotation;
		static const FLazyName GradientSmoothness;

		// Texture
		static const FLazyName MainTexture;
		static const FLazyName TexturedUTiling;
		static const FLazyName TexturedVTiling;

		// Bounds
		static const FLazyName BoundsOrigin;
		static const FLazyName BoundsSize;
		static const FLazyName TextPosition;

		// Mask
		static const FLazyName MaskEnabled;
		static const FLazyName MaskRotation;
		static const FLazyName MaskOffset;
		static const FLazyName MaskSmoothness;
	};

	UText3DProjectSettings();

	TEXT3D_API UMaterial* GetDefaultMaterial() const;
	TEXT3D_API UText3DStyleSet* GetDefaultStyleSet() const;
	TEXT3D_API UFont* GetFallbackFont() const;
	TEXT3D_API UFontFace* GetFallbackFontFace() const;
	TEXT3D_API const FString& GetFontDirectory() const;

	TEXT3D_API UMaterial* GetBaseMaterial(const FText3DMaterialKey& InKey) const;

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand")
	TEXT3D_API void SetScheduleFontFaceGlyphCleanup(bool bInEnabled);

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand")
	TEXT3D_API bool GetScheduleFontFaceGlyphCleanup() const;

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand")
	TEXT3D_API void SetFontFaceGlyphCleanupPeriod(float InValue);

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand")
	TEXT3D_API float GetFontFaceGlyphCleanupPeriod() const;

	/** Sets the amount of time, in seconds, to wait before a cleanup happens after a mesh becomes unused  */
	TEXT3D_API void SetFontFaceGlyphCleanupDelay(float InDelay);

	/** The amount of time, in seconds, to wait before a cleanup happens after a mesh becomes unused  */
	float GetFontFaceGlyphCleanupDelay() const
	{
		return FontFaceGlyphCleanupDelay;
	}

#if WITH_EDITOR
	void OpenEditorSettingsWindow() const;
	TEXT3D_API TConstArrayView<FString> GetFavoriteFonts() const;
	TEXT3D_API bool GetShowOnlyMonospaced() const;
	TEXT3D_API bool GetShowOnlyBold() const;
	TEXT3D_API bool GetShowOnlyItalic() const;

	TEXT3D_API void AddFavoriteFont(const FString& InFontName);
	TEXT3D_API void RemoveFavoriteFont(const FString& InFontName);
	TEXT3D_API void SetShowOnlyMonospaced(bool InShowOnlyMonospaced);
	TEXT3D_API void SetShowOnlyBold(bool InShowOnlyBold);
	TEXT3D_API void SetShowOnlyItalic(bool InShowOnlyItalic);

	bool GetDebugMode() const;
#endif

	/** Returns whether Text3D collision should be forced disabled */
	TEXT3D_API bool ShouldForceDisableTextCollision() const;

	bool GetUseExistingCharactersAsTemplates() const
	{
		return bUseExistingCharactersAsTemplates;
	}

protected:
	void AddMaterial(FText3DMaterialKey&& InKey, TSoftObjectPtr<UMaterial>&& InMaterial);

	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InChangeEvent) override;
#endif
	//~ End UObject

#if WITH_EDITOR
	void Save();
#endif

	/** Default custom material used on Text */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	TSoftObjectPtr<UMaterial> DefaultMaterial;

	/** Default style rules to apply on a new Text3D instance */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	TSoftObjectPtr<UText3DStyleSet> DefaultStyleSet;
	
	/** Default font used on new instance or when the selected font is unavailable */
	UPROPERTY(Config, EditAnywhere, DisplayName="Default Font", Category="Text3D")
	TSoftObjectPtr<UFont> FallbackFont;

	/** Font face used as fallback when no font faces are found */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	TSoftObjectPtr<UFontFace> FallbackFontFace;

	/** Default project directory where system fonts will be imported and stored as UFont */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	FString FontDirectory;

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand")
	UPROPERTY(meta=(DeprecatedProperty))
	bool bScheduleFontFaceGlyphCleanup_DEPRECATED = true;

	UE_DEPRECATED(5.8, "Font face glyph fixed scheduled cleanup is deprecated. Glyphs are cleaned up on-demand. Use FontFaceGlyphCleanupDelay instead")
	UPROPERTY(meta=(DeprecatedProperty))
	float FontFaceGlyphCleanupPeriod_DEPRECATED = 300.f;

	/** Once a glyph mesh becomes unused, how long to wait before a cleanup happens. <= 0 means immediate. */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	float FontFaceGlyphCleanupDelay = 5.f;

	/** 
	 * Option to disable Text3D collisions globally, as Text3D with collisions can have a significant hit on perf. 
	 * NOTE: Enabling this and saving levels with text3D will leave them in the 'No Collision' state.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	bool bForceDisableTextCollision = false;

	/** Mirror atomic version of bForceDisableTextCollision */
	std::atomic<bool> ForceDisableTextCollision;

	/**
	 * If enabled, when generating components for characters, attempt to use existing character components as templates.
	 * New characters components will duplicate from these existing character components, rather than constructing default ones.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Text3D Renderer")
	bool bUseExistingCharactersAsTemplates = true;

#if WITH_EDITORONLY_DATA
	/** Favorites fonts pinned in the font viewer dropdown */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	TArray<FString> FavoriteFonts;

	/** Only show monospaced fonts in font viewer */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	bool bShowOnlyMonospaced = false;

	/** Only show fonts with bold support in font viewer */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	bool bShowOnlyBold = false;

	/** Only show fonts with italic support in font viewer */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	bool bShowOnlyItalic = false;

	/** System font names retrieved from the current platform */
	UPROPERTY(VisibleAnywhere, Transient, AdvancedDisplay, Category="Text3D")
	TArray<FString> SystemFontNames;

	/** Debug Text3D components tree hierarchy */
	UPROPERTY(Config, EditAnywhere, Category="Text3D")
	bool bDebugMode = false;
#endif

	/** Base parent material */
	UPROPERTY(Transient)
	TMap<FText3DMaterialKey, TSoftObjectPtr<UMaterial>> BaseMaterials;
};
