// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Styling/SlateTypes.h"
#include "Text3DTypes.h"
#include "UObject/Object.h"
#include "Text3DStyleBase.generated.h"

class UFont;
struct FTextBlockStyle;
struct FTypefaceEntry;

/** Text3D rich text format styles */
UCLASS(MinimalAPI, EditInlineNew, ClassGroup=Text3D, DisplayName="FormatStyle", AutoExpandCategories=(Style))
class UText3DStyleBase : public UObject
{
	GENERATED_BODY()

public:
	UText3DStyleBase();

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetStyleName(FName InName);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	FName GetStyleName() const
	{
		return StyleName;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetOverrideFont(bool bInOverride);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	bool GetOverrideFont() const
	{
		return bOverrideFont;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetFont(UFont* InFont);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	UFont* GetFont() const
	{
		return Font;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetFontTypeface(FName InTypeface);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	FName GetFontTypeface() const
	{
		return FontTypeface;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetOverrideFontSize(bool bInOverride);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	bool GetOverrideFontSize() const
	{
		return bOverrideFontSize;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetFontSize(float InSize);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	float GetFontSize() const
	{
		return FontSize;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetOverrideFrontColor(bool bInOverride);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	bool GetOverrideFrontColor() const
	{
		return bOverrideFrontColor;
	}

	UFUNCTION(BlueprintCallable, Category = "Text3D|Style")
	TEXT3D_API void SetFrontColor(FLinearColor InColor);

	UFUNCTION(BlueprintPure, Category = "Text3D|Style")
	FLinearColor GetFrontColor() const
	{
		return FrontColor;
	}

	/** Sets the proper values on the text block style */
	void ConfigureStyle(FTextBlockStyle& OutStyle) const;

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnStylePropertiesChanged();
	void OnFontChanged();

	UFUNCTION()
	TArray<FName> GetTypefaceNames() const;
	TArrayView<const FTypefaceEntry> GetAvailableTypefaces() const;

	/** Style name used in text */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(AllowPrivateAccess="true"))
	FName StyleName = NAME_None;

	/** Behavior for the font rule of this style */
	UPROPERTY(EditAnywhere, Setter="SetOverrideFont", Getter="GetOverrideFont", Category="Style", meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideFont = true;

	/** Text font defines the style of rendered characters */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(EditCondition="bOverrideFont", AdvancedFontPicker, AllowPrivateAccess="true"))
	TObjectPtr<UFont> Font;

	/** Text font face, subset within font like bold, italic, regular */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(EditCondition="bOverrideFont", GetOptions="GetTypefaceNames", AllowPrivateAccess="true"))
	FName FontTypeface;

	/** Behavior for the size rule of this style */
	UPROPERTY(EditAnywhere, Setter="SetOverrideFontSize", Getter="GetOverrideFontSize", Category="Style", meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideFontSize = true;

	/** Text font size to render and scale each character */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(ClampMin="0.1", EditCondition="bOverrideFontSize", AllowPrivateAccess="true"))
	float FontSize = 48.f;

	/** Behavior for the front color rule of this style */
	UPROPERTY(EditAnywhere, Setter="SetOverrideFrontColor", Getter="GetOverrideFrontColor", Category="Style", meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideFrontColor = true;

	/** Text front color */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Style", meta=(EditCondition="bOverrideFrontColor", AllowPrivateAccess="true"))
	FLinearColor FrontColor = FLinearColor::White;

	
};
