// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Text3DStyleSet.generated.h"

class UText3DStyleBase;

/** Text3D Style Set used to store multiple reusable styles */
UCLASS(MinimalAPI, DisplayName="Text3D Style Set", AutoExpandCategories=(Style))
class UText3DStyleSet : public UObject
{
	GENERATED_BODY()

	friend class UText3DStyleBase;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStyleSetUpdated, UText3DStyleSet* /** InStyleSet */)

	static FOnStyleSetUpdated::RegistrationType& OnStyleSetUpdated()
	{
		return OnStyleSetUpdatedDelegate;
	}

	TConstArrayView<TObjectPtr<UText3DStyleBase>> GetStyles() const
	{
		return Styles;
	}

protected:
	static FOnStyleSetUpdated OnStyleSetUpdatedDelegate;

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnStyleSetPropertiesChanged();

	UPROPERTY(EditAnywhere, Instanced, Category="Style", NoClear, meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UText3DStyleBase>> Styles;
};