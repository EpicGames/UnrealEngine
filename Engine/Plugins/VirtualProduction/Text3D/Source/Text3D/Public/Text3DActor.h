// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Text3DActor.generated.h"

class UText3DComponent;

UCLASS(ClassGroup = (Text3D), Meta = (DisplayName = "Text 3D", ComponentWrapperClass))
class TEXT3D_API AText3DActor : public AActor
{
	GENERATED_BODY()

public:
	AText3DActor();

	/** Returns Text3D subobject **/
	UText3DComponent* GetText3DComponent() const
	{
		return Text3DComponent;
	}

	//~ Begin AActor
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

private:
	UPROPERTY(Category = Text, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UText3DComponent> Text3DComponent;
};
