// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Text3DTypes.h"
#include "Containers/AdderRef.h"
#include "Text3DComponent.generated.h"

namespace UE::Text3D
{
	class FTextBuilder;
}

class UFont;
class UMaterialInterface;
class UText3DCharacterBase;
class UText3DCharacterExtensionBase;
class UText3DExtensionBase;
class UText3DGeometryExtensionBase;
class UText3DLayoutEffectBase;
class UText3DLayoutExtensionBase;
class UText3DMaterialExtensionBase;
class UText3DRendererBase;
class UText3DRenderingExtensionBase;
class UText3DStyleExtensionBase;
class UText3DTokenExtensionBase;
struct FTypefaceEntry;

UCLASS(MinimalAPI
	, ClassGroup = (Text3D)
	, PrioritizeCategories="Text Character Layout Geometry Material Rendering Transform"
	, HideCategories=(Replication,Collision,HLOD,Physics,Networking,Input,Actor,Cooking,LevelInstance,Streaming,DataLayers,WorldPartition)
	, meta = (BlueprintSpawnableComponent))
class UText3DComponent : public USceneComponent
{
	// Needed to support older API (see FormatText)
	friend class UText3DDefaultTokenExtension;

	GENERATED_BODY()

public:
	TEXT3D_API UText3DComponent();

	/**
	 * Delegate called after text is rebuilt
	 */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTextGenerated);

	/**
	 * Delegate called after text is rebuilt
	 */
	DECLARE_MULTICAST_DELEGATE(FTextGeneratedNative);
	FTextGeneratedNative::RegistrationType& OnTextGenerated()
	{
		return TextGeneratedNativeDelegate;
	}
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FTextUpdated, UText3DComponent*, EText3DRendererFlags);
	FTextUpdated::RegistrationType& OnTextPostUpdate()
	{
		return TextPostUpdateDelegate;
	}

	FTextUpdated::RegistrationType& OnTextPreUpdate()
	{
		return TextPreUpdateDelegate;
	}

	/** Get the text value */
	UFUNCTION(BlueprintPure, BlueprintGetter, Category = "Text3D")
	const FText& GetText() const
	{
		return Text;
	}

	/** Set the text value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Text3D")
	TEXT3D_API void SetText(const FText& Value);

	/**
	 * Returns the Text property, after being formatted by the FormatText virtual function.
	 * If FormatText is not overriden, the return FText will be the same as the Text property.
	 */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API const FText& GetFormattedText() const;

	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Text3D")
	TEXT3D_API void SetEnforceUpperCase(bool bInEnforceUpperCase);

	UFUNCTION(BlueprintPure, BlueprintGetter, Category = "Text3D")
	bool GetEnforceUpperCase() const
	{
		return bEnforceUpperCase;
	}

	/** Get the text font */
	UFUNCTION(BlueprintPure, BlueprintGetter, Category = "Text3D")
	UFont* GetFont() const
	{
		return Font;
	}

	/** Set the text font and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Text3D")
	TEXT3D_API void SetFont(UFont* InFont);

	/** Set the text font size */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetFontSize(float InSize);

	UFUNCTION(BlueprintPure, Category = "Text3D")
	TEXT3D_API float GetFontSize() const;

	/** Set whether an outline is applied. */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetHasOutline(const bool bValue);

	/** Set the outline width. */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetOutlineExpand(const float Value);

	/** Set the text extrusion size and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetExtrude(const float Value);

	/** Set the 3d bevel value */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetBevel(const float Value);

	/** Set the 3d bevel type */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetBevelType(const EText3DBevelType Value);

	/** Set the amount of segments that will be used to tessellate the Bevel */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetBevelSegments(const int32 Value);

	/** Get the text front material */
	TEXT3D_API UMaterialInterface* GetFrontMaterial() const;

	/** Set the text front material */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetFrontMaterial(UMaterialInterface* Value);

	/** Get the text bevel material */
	TEXT3D_API UMaterialInterface* GetBevelMaterial() const;

	/** Set the text bevel material */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetBevelMaterial(UMaterialInterface* Value);
	
	/** Get the text extrude material */
	TEXT3D_API UMaterialInterface* GetExtrudeMaterial() const;

	/** Set the text extrude material */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetExtrudeMaterial(UMaterialInterface* Value);

	/** Get the text back material */
	TEXT3D_API UMaterialInterface* GetBackMaterial() const;

	/** Set the text back material */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetBackMaterial(UMaterialInterface* Value);

	/** Get the kerning value and signal the primitives to be rebuilt */
	TEXT3D_API float GetKerning() const;

	/** Set the kerning value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetKerning(const float Value);

	/** Get the line spacing value and signal the primitives to be rebuilt */
	TEXT3D_API float GetLineSpacing() const;

	/** Set the line spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetLineSpacing(const float Value);

	/** Get the word spacing value and signal the primitives to be rebuilt */
	TEXT3D_API float GetWordSpacing() const;

	/** Set the word spacing value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetWordSpacing(const float Value);

	/** Get the horizontal alignment value and signal the primitives to be rebuilt */
	TEXT3D_API EText3DHorizontalTextAlignment GetHorizontalAlignment() const;

	/** Set the horizontal alignment value and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetHorizontalAlignment(const EText3DHorizontalTextAlignment value);

	/** Get the vertical alignment and signal the primitives to be rebuilt */
	TEXT3D_API EText3DVerticalTextAlignment GetVerticalAlignment() const;

	/** Set the vertical alignment and signal the primitives to be rebuilt */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetVerticalAlignment(const EText3DVerticalTextAlignment value);

	/** Whether a maximum width is specified */
	TEXT3D_API bool HasMaxWidth() const;

	/** Enable / Disable a Maximum Width */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetHasMaxWidth(const bool Value);

	/** Get the Maximum Width - If width is larger, mesh will scale down to fit MaxWidth value */
	TEXT3D_API float GetMaxWidth() const;

	/** Set the Maximum Width - If width is larger, mesh will scale down to fit MaxWidth value */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetMaxWidth(const float Value);

	/** Get the Maximum Width Handling - Whether to wrap before scaling when the text size reaches the max width */
	TEXT3D_API EText3DMaxWidthHandling GetMaxWidthHandling() const;

	/** Set the Maximum Width Handling - Whether to wrap before scaling when the text size reaches the max width */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetMaxWidthHandling(const EText3DMaxWidthHandling Value);

	/** Whether a maximum height is specified */
	TEXT3D_API bool HasMaxHeight() const;

	/** Enable / Disable a Maximum Height */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetHasMaxHeight(const bool Value);

	/** Get the Maximum Height - If height is larger, mesh will scale down to fit MaxHeight value */
	TEXT3D_API float GetMaxHeight() const;

	/** Set the Maximum Height - If height is larger, mesh will scale down to fit MaxHeight value */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetMaxHeight(const float Value);

	/** Get if the mesh should scale proportionally when Max Width/Height is set */
	TEXT3D_API bool ScalesProportionally() const;

	/** Set if the mesh should scale proportionally when Max Width/Height is set */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetScaleProportionally(const bool Value);

	/** Get the value of CastShadow. */
	TEXT3D_API bool CastsShadow() const;

	/** Set the value of CastShadow. */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void SetCastShadow(bool NewCastShadow);

	/** Get whole text rendered bounds */
	UFUNCTION(BlueprintCallable, Category = "Text3D")
	TEXT3D_API void GetBounds(FVector& Origin, FVector& BoxExtent) const;

	/** Get whole text rendered bounds */
	FBox GetBounds() const;

	/** Get the typeface */
	UFUNCTION(BlueprintPure, BlueprintGetter, Category = "Text3D")
	FName GetTypeface() const
	{
		return Typeface;
	}

	/** Set the typeface */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Text3D")
	TEXT3D_API void SetTypeface(const FName InTypeface);

	void RefreshTypeface();

	UFUNCTION(BlueprintCallable, Category = "Text3D")
	void SetTextRendererClass(const TSubclassOf<UText3DRendererBase>& InClass);

	UFUNCTION(BlueprintPure, Category = "Text3D")
	TSubclassOf<UText3DRendererBase> GetTextRendererClass() const
	{
		return TextRendererClass;
	}

	const FText3DStatistics& GetStatistics() const
	{
		return Statistics;
	}

	TEXT3D_API void RequestUpdate(EText3DRendererFlags InFlags, bool bInImmediateUpdate=false);

	int32 GetTypeFaceIndex() const;

	const FTypefaceEntry* GetTypeFaceEntry() const;
	
	TEXT3D_API UText3DRendererBase* GetTextRenderer() const;

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InRendererClass"))
	UText3DRendererBase* GetTextRenderer(TSubclassOf<UText3DRendererBase> InRendererClass) const;

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DTokenExtensionBase* GetTokenExtension(TSubclassOf<UText3DTokenExtensionBase> InExtensionClass);
	
	UText3DTokenExtensionBase* GetTokenExtension() const
	{
		return TokenExtension;
	}
	
	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DTokenExtensionBase>::Value)>
	T* GetCastedTokenExtension() const
	{
		return Cast<T>(TokenExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DStyleExtensionBase* GetStyleExtension(TSubclassOf<UText3DStyleExtensionBase> InExtensionClass);
	
	UText3DStyleExtensionBase* GetStyleExtension() const
	{
		return StyleExtension;
	}
	
	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DStyleExtensionBase>::Value)>
	T* GetCastedStyleExtension() const
	{
		return Cast<T>(StyleExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DLayoutExtensionBase* GetLayoutExtension(TSubclassOf<UText3DLayoutExtensionBase> InExtensionClass);

	UText3DLayoutExtensionBase* GetLayoutExtension() const
	{
		return LayoutExtension;
	}

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DLayoutExtensionBase>::Value)>
	T* GetCastedLayoutExtension() const
	{
		return Cast<T>(LayoutExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DMaterialExtensionBase* GetMaterialExtension(TSubclassOf<UText3DMaterialExtensionBase> InExtensionClass);

	UText3DMaterialExtensionBase* GetMaterialExtension() const
	{
		return MaterialExtension;
	}

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DMaterialExtensionBase>::Value)>
	T* GetCastedMaterialExtension() const
	{
		return Cast<T>(MaterialExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DGeometryExtensionBase* GetGeometryExtension(TSubclassOf<UText3DGeometryExtensionBase> InExtensionClass);

	UText3DGeometryExtensionBase* GetGeometryExtension() const
	{
		return GeometryExtension;
	}

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DGeometryExtensionBase>::Value)>
	T* GetCastedGeometryExtension() const
	{
		return Cast<T>(GeometryExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InExtensionClass"))
	UText3DRenderingExtensionBase* GetRenderingExtension(TSubclassOf<UText3DRenderingExtensionBase> InExtensionClass);
	
	UText3DRenderingExtensionBase* GetRenderingExtension() const
	{
		return RenderingExtension;
	}
	
	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DRenderingExtensionBase>::Value)>
	T* GetCastedRenderingExtension() const
	{
		return Cast<T>(RenderingExtension);
	}

	UFUNCTION(BlueprintCallable, Category="Text3D", meta=(DeterminesOutputType="InEffectClass"))
	TArray<UText3DLayoutEffectBase*> GetLayoutEffects(TSubclassOf<UText3DLayoutEffectBase> InEffectClass);

	TConstArrayView<UText3DLayoutEffectBase*> GetLayoutEffects() const
	{
		return LayoutEffects;
	}
	
	UText3DCharacterExtensionBase* GetCharacterExtension() const
	{
		return CharacterExtension;
	}
	
	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DCharacterExtensionBase>::Value)>
	T* GetCastedCharacterExtension() const
	{
		return Cast<T>(CharacterExtension);
	}

	TEXT3D_API uint16 GetCharacterCount() const;

	TEXT3D_API UText3DCharacterBase* GetCharacter(uint16 InCharacterIndex) const;

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DCharacterBase>::Value)>
	T* GetCastedCharacter(uint16 InCharacterIndex) const
	{
		return Cast<T>(GetCharacter(InCharacterIndex));
	}

	TEXT3D_API void ForEachCharacter(const TFunctionRef<void(UText3DCharacterBase*, uint16, uint16)>& InFunctor) const;

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UText3DCharacterBase>::Value)>
	void ForEachCastedCharacter(const TFunctionRef<void(T*, uint16, uint16)>& InFunctor) const
	{
		ForEachCharacter([&InFunctor](UText3DCharacterBase* InCharacter, uint16 InCharacterIndex, uint16 InCharacterCount)
		{
			InFunctor(Cast<T>(InCharacter), InCharacterIndex, InCharacterCount);
		});
	}

	/** Retrieves all the extensions for this component (not sorted) */
	bool GatherExtensions(TAdderReserverRef<UText3DExtensionBase*> OutExtensions) const;

	/** Prepares the component for a build */
	bool BeginBuild(TNotNull<EText3DRendererFlags*> InOutFlags);

	/** Called when the text has finished building */
	void FinishBuild(EText3DRendererFlags InFlags);

	/** Called when the build is cancelled */
	void CancelBuild();

	/** 
	 * Determines whether text3d is using blocking builds.
	 * If true, the builds are blocking (non-incrementally). Otherwise builds are done incrementally. 
	 * When true, this does not mean that the build will be executed immediately, just that when it does execute, it will be performed non-incrementally (i.e. finish in that same frame).
	 */
	UFUNCTION(BlueprintPure, BlueprintGetter, Category = "Text3D")
	bool GetUseBlockingBuild() const
	{
		return bUseBlockingBuild;
	}

	/** 
	 * Set whether to use blocking builds.
	 * If true, the builds are blocking (non-incrementally). Otherwise builds are done incrementally. 
	 * When true, this does not mean that the build will be executed immediately, just that when it does execute, it will be performed non-incrementally (i.e. finish in that same frame).
	 */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Text3D")
	TEXT3D_API void SetUseBlockingBuild(bool bInUseBlockingBuild);

protected:
	//~ Begin USceneComponent
	/** Intercept and propagate a change on this component to all children. */
	TEXT3D_API virtual void OnVisibilityChanged() override;
	/** Intercept and propagate a change on this component to all children. */
	TEXT3D_API virtual void OnHiddenInGameChanged() override;
	//~ End USceneComponent

	//~ Begin UActorComponent
	TEXT3D_API virtual void OnComponentCreated() override;
	TEXT3D_API virtual void OnComponentDestroyed(bool bInDestroyingHierarchy) override;
	TEXT3D_API virtual void OnRegister() override;
	TEXT3D_API virtual void OnUnregister() override;
	TEXT3D_API virtual void PostApplyToComponent() override;
	TEXT3D_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent

	//~ Begin UObject
	TEXT3D_API virtual void Serialize(FArchive& InArchive) override;
	TEXT3D_API virtual void PostLoad() override;
	TEXT3D_API virtual void PostEditImport() override;
	TEXT3D_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	TEXT3D_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& InContext) override;
	TEXT3D_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	TEXT3D_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject

#if WITH_EDITOR
	friend class UText3DEditorFontSubsystem;

	DECLARE_DELEGATE_RetVal_OneParam(UFont*, FOnResolveFontByName, const FString& InFontName)
	static TEXT3D_API FOnResolveFontByName OnResolveFontByNameDelegate;

	/** Legacy : used to resolve font by their names */
	TEXT3D_API void SetFontByName(const FString& InFontName);

	/** Refresh text to force update render state */
	UFUNCTION(CallInEditor, Category="Utilities")
	void ForceUpdateText();

	/** Open Text project settings editor window */
	UFUNCTION(CallInEditor, Category="Utilities")
	void OpenProjectSettings();
#endif

	/** Get the type faces supported for the current font */
	UFUNCTION()
	TArray<FName> GetTypefaceNames() const;

	/**
	 * Will be called when text geometry is generated.
	 * Override it to customize text formatting in the final geometry, without affecting the Text property.
	 * Use GetFormattedText() to retrieve a FText with the result of this formatting.
	 */
	TEXT3D_API virtual void FormatText(FText& InOutText) const;

	/** Called when TextRendererClass is changed */
	void OnTextRendererClassChanged();

	/** Called when Text is changed */
	void OnTextChanged();

	/** Called when font options are changed */
	void OnFontPropertiesChanged();

	/** The renderer class to use to create the text geometry */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Text", NoClear, meta=(ShowDisplayNames, AllowPrivateAccess="true"))
	TSubclassOf<UText3DRendererBase> TextRendererClass;

	/** The text to generate a 3d mesh */
	UPROPERTY(EditAnywhere, Getter, Setter, BlueprintSetter="SetText", BlueprintGetter="GetText", Category = "Text", meta = (MultiLine = true, AllowPrivateAccess = "true"))
	FText Text;

	/** Text font defines the style of rendered characters */
	UPROPERTY(EditAnywhere, Getter, Setter, BlueprintSetter="SetFont", BlueprintGetter="GetFont", DisplayName = "Default Font", Category = "Text", meta = (AdvancedFontPicker, AllowPrivateAccess = "true"))
	TObjectPtr<UFont> Font;

	/** Text font face, subset within font like bold, italic, regular */
	UPROPERTY(EditAnywhere, Getter, Setter, BlueprintSetter="SetTypeface", BlueprintGetter="GetTypeface", DisplayName = "Default Typeface", Category = "Text", meta = (AllowPrivateAccess = "true", GetOptions="GetTypefaceNames"))
	FName Typeface;

	/** Text font size defines the size of each rendered character */
	UPROPERTY(EditAnywhere, Getter, Setter, DisplayName = "Default FontSize", Category = "Text", meta=(ClampMin = "0.1", AllowPrivateAccess = "true"))
	float FontSize = 48.f;

	/** Whether to force upper case for text */
	UPROPERTY(EditAnywhere, Getter="GetEnforceUpperCase", Setter="SetEnforceUpperCase", BlueprintSetter="SetEnforceUpperCase", BlueprintGetter="GetEnforceUpperCase", Category = "Text", meta = (AllowPrivateAccess="true"))
	bool bEnforceUpperCase = false;

	/** 
	 * Whether to use blocking builds for text by default.
	 * If true, the builds are blocking (non-incrementally). Otherwise builds are done incrementally. 
	 * When true, this does not mean that the build will be executed immediately, just that when it does execute, it will be performed non-incrementally (i.e. finish in that same frame).
	 */
	UPROPERTY(EditAnywhere, Getter="GetUseBlockingBuild", BlueprintGetter="GetUseBlockingBuild", Setter="SetUseBlockingBuild", BlueprintSetter="SetUseBlockingBuild", Category="Text", meta=(AllowPrivateAccess="true"))
	bool bUseBlockingBuild = false;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Token", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DTokenExtensionBase> TokenExtension;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Style", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DStyleExtensionBase> StyleExtension;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Character", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DCharacterExtensionBase> CharacterExtension;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Layout", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DLayoutExtensionBase> LayoutExtension;

	UPROPERTY(EditAnywhere, Instanced, Category="LayoutEffects", meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UText3DLayoutEffectBase>> LayoutEffects;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Geometry", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DGeometryExtensionBase> GeometryExtension;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Material", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DMaterialExtensionBase> MaterialExtension;

	UPROPERTY(VisibleAnywhere, Instanced, Category="Rendering", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UText3DRenderingExtensionBase> RenderingExtension;

private:
	UPROPERTY(VisibleAnywhere, Instanced, NoClear, Category="Text", meta=(AllowPrivateAccess="true", NoResetToDefault))
	TObjectPtr<UText3DRendererBase> TextRenderer;

	UPROPERTY(BlueprintAssignable, Category=Events, meta=(AllowPrivateAccess="true", DisplayName="On Text Generated"))
	FTextGenerated TextGeneratedDelegate;

	/** Accumulated update flags to trigger on OnRegister() */
	EText3DRendererFlags PreRegisterUpdateFlags = EText3DRendererFlags::None;

	FTextGeneratedNative TextGeneratedNativeDelegate;

	/** Called before text is updated */
	FTextUpdated TextPreUpdateDelegate;

	/** Called after text is updated */
	FTextUpdated TextPostUpdateDelegate;

	/** Flagged as true when text is being updated */
	bool bIsUpdatingText = false;

	/** Text statistics cached since last text generation */
	FText3DStatistics Statistics;

	bool IsTypefaceAvailable(FName InTypeface) const;
	TArrayView<const FTypefaceEntry> GetAvailableTypefaces() const;

	void UpdateStatistics();

	void OnComponentTransformed(USceneComponent*, EUpdateTransformFlags, ETeleportType);

	bool IsCompiledFromBlueprint() const;

	bool bZeroScaled = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> FrontMaterial_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> BevelMaterial_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> ExtrudeMaterial_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> BackMaterial_DEPRECATED;
	
	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	float Extrude_DEPRECATED = 5.0f;

	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	float Bevel_DEPRECATED = 0.0f;

	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	EText3DBevelType BevelType_DEPRECATED = EText3DBevelType::Convex;
		
	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	int32 BevelSegments_DEPRECATED = 8;

	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	bool bOutline_DEPRECATED = false;

	UE_DEPRECATED(5.6, "Use Geometry Extension instead")
	UPROPERTY()
	float OutlineExpand_DEPRECATED = 0.5f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	EText3DHorizontalTextAlignment HorizontalAlignment_DEPRECATED = EText3DHorizontalTextAlignment::Left;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	EText3DVerticalTextAlignment VerticalAlignment_DEPRECATED = EText3DVerticalTextAlignment::FirstLine;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	float Kerning_DEPRECATED = 0.0f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	float LineSpacing_DEPRECATED = 0.0f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	float WordSpacing_DEPRECATED = 0.0f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	bool bHasMaxWidth_DEPRECATED = false;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	float MaxWidth_DEPRECATED = 500.f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	bool bHasMaxHeight_DEPRECATED = false;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	EText3DMaxWidthHandling MaxWidthHandling_DEPRECATED = EText3DMaxWidthHandling::Scale;
	
	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	float MaxHeight_DEPRECATED = 500.0f;

	UE_DEPRECATED(5.6, "Use Layout Extension instead")
	UPROPERTY()
	bool bScaleProportionally_DEPRECATED = true;
	
	UE_DEPRECATED(5.6, "Use Rendering Extension instead")
	UPROPERTY()
	bool bCastShadow_DEPRECATED = true;
#endif

	friend class UE::Text3D::FTextBuilder;
	friend struct FText3DComponentInstanceData;
};
