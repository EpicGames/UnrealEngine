// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

/** Used for versioning, only used internally */
struct FText3DComponentVersion
{
	enum Type : int32
	{
		PreVersioning = 0,
		/**
		 * Full refactoring of Text3D, splitting logic into extensions to be reused across multiple renderers
		 * Geometry, Layout, Material, Rendering
		 * Integrating children class (AvaText3DComponent) features into native Text3D
		 */
		Extensions,
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x9261D8A4, 0xBF424601, 0xA21FC0A3, 0x9D839565);
};