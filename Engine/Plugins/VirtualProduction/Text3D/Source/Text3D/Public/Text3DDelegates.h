// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Text3DTypes.h"

#define UE_API TEXT3D_API

namespace UE::Text3D
{

struct UE_INTERNAL FText3DBuildContext
{
	explicit FText3DBuildContext(const Renderer::FUpdateParameters& InUpdateParameters);

	/** All dirty flags for this update */
	EText3DRendererFlags UpdateFlags = EText3DRendererFlags::None;
};

/** Called before the Text3D renderer update */
void BroadcastPreRendererUpdate(const FText3DBuildContext& InBuildContext);
/** Called after the Text3D renderer update */
void BroadcastPostRendererUpdate(const FText3DBuildContext& InBuildContext);

/** Delegate called before the Text3D renderer update */
UE_INTERNAL UE_API TMulticastDelegateRegistration<void(const FText3DBuildContext&)>& OnText3DPreRendererUpdate();
/** Delegate called after the Text3D renderer update */
UE_INTERNAL UE_API TMulticastDelegateRegistration<void(const FText3DBuildContext&)>& OnText3DPostRendererUpdate();

} // UE::Text3D

#undef UE_API
