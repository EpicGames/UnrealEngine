// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Box.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectPtr.h"
#include "Text3DTypes.generated.h"

class UDynamicMesh;
class UMaterialInterface;
class UStaticMesh;
class UWorld;
struct FText3DFontFaceCache;

/** Enumerate Text3D update flags based on their priority */
enum class EText3DRendererFlags : uint8
{
	None,
	/** Update whole geometry for text */
	Geometry = 1 << 0,
	/** Update layout for characters (transform) */
	Layout = 1 << 1,
	/** Update materials slots */
	Material = 1 << 2,
	/** Update visibility/lighting properties */
	Visibility = 1 << 3,
	/** Update everything */
	All = Geometry | Layout | Visibility | Material
};
ENUM_CLASS_FLAGS(EText3DRendererFlags);

UENUM()
enum class EText3DMaterialStyle : uint8 
{
	Invalid UMETA(Hidden),
	Solid,
	Gradient,
	Texture,
	Custom
};

UENUM()
enum class EText3DMaterialBlendMode : uint8
{
	Invalid UMETA(Hidden),
	Opaque,
	Translucent
};

UENUM()
enum class EText3DFontStyleFlags : uint8
{
	None = 0,
	Monospace = 1 << 0,
	Bold = 1 << 1,
	Italic = 1 << 2,
};

UENUM()
enum class EText3DBevelType : uint8
{
	Linear,
	HalfCircle,
	Convex,
	Concave,
	OneStep,
	TwoSteps,
	Engraved
};

UENUM()
enum class EText3DOutlineType : uint8
{
	Stroke,
	Fill
};

UENUM()
enum class EText3DMeshType : uint8
{
	Unknown,
	Static,
	Dynamic,
};

UENUM()
enum class EText3DGroupType : uint8
{
	Front = 0,
	Bevel = 1,
	Extrude = 2,
	Back = 3,

	TypeCount = 4
};

UENUM()
enum class EText3DVerticalTextAlignment : uint8
{
	FirstLine		UMETA(DisplayName = "First Line"),
	Top				UMETA(DisplayName = "Top"),
	Center			UMETA(DisplayName = "Center"),
	Bottom			UMETA(DisplayName = "Bottom"),
};

UENUM()
enum class EText3DHorizontalTextAlignment : uint8
{
	Left			UMETA(DisplayName = "Left"),
	Center			UMETA(DisplayName = "Center"),
	Right			UMETA(DisplayName = "Right"),
};

UENUM()
enum class EText3DMaxWidthHandling : uint8
{
	/** Scales the text to meet the max width */
	Scale			UMETA(DisplayName = "Scale"),
	/** First wraps the text (if possible) and then scales to meet the max width */
	WrapAndScale	UMETA(DisplayName = "Wrap and Scale"),
};

UENUM()
enum class EText3DCharacterEffectOrder : uint8
{
	Normal			UMETA(DisplayName = "Left To Right"),
	FromCenter		UMETA(DisplayName = "From Center"),
	ToCenter		UMETA(DisplayName = "To Center"),
	Opposite		UMETA(DisplayName = "Right To Left"),
};

struct FText3DWordStatistics
{
	/** Actual range taking into account whitespaces */
	FTextRange ActualRange;

	/** Render range not taking into account whitespaces */
	FTextRange RenderRange;
};

struct FText3DStatistics
{
	TArray<FText3DWordStatistics> Words;
	int32 WhiteSpaces;
};

struct FText3DFontFamily
{
	void AddFontFace(const FString& InFontFaceName, const FString& InFontFacePath)
	{
		if (FontFacePaths.Contains(InFontFaceName))
		{
			return;
		}

		FontFacePaths.Add(InFontFaceName, InFontFacePath);
	}

	/** Family these font faces belong to */
	FString FontFamilyName;

	/** Map of each font face with name -> path */
	TMap<FString, FString> FontFacePaths;
};

/** Used to identify a specific material type */
USTRUCT()
struct FText3DMaterialKey
{
	GENERATED_BODY()

	FText3DMaterialKey() = default;
	explicit FText3DMaterialKey(EText3DMaterialBlendMode InBlend, bool bInIsUnlit)
		: BlendMode(InBlend)
		, bIsUnlit(bInIsUnlit)
	{}
		
	bool operator==(const FText3DMaterialKey& Other) const
	{
		return BlendMode == Other.BlendMode
			&& bIsUnlit == Other.bIsUnlit;
	}

	bool operator!=(const FText3DMaterialKey& Other) const
	{
		return !(*this == Other);
	}
	
	friend uint32 GetTypeHash(const FText3DMaterialKey& InMaterialSettings)
	{
		return HashCombineFast(GetTypeHash(InMaterialSettings.BlendMode), GetTypeHash(InMaterialSettings.bIsUnlit));
	}

	UPROPERTY()
	EText3DMaterialBlendMode BlendMode = EText3DMaterialBlendMode::Invalid;

	UPROPERTY()
	bool bIsUnlit = false;
};

USTRUCT()
struct FText3DMaterialGroupKey
{
	GENERATED_BODY()

	FText3DMaterialGroupKey() = default;
	explicit FText3DMaterialGroupKey(FText3DMaterialKey InKey, EText3DGroupType InGroup, FName InTag)
		: Key(InKey)
		, Group(InGroup)
		, Tag(InTag)
	{}

	bool operator==(const FText3DMaterialGroupKey& Other) const
	{
		return Key == Other.Key
			&& Group == Other.Group
			&& Tag == Other.Tag;
	}

	bool operator!=(const FText3DMaterialGroupKey& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FText3DMaterialGroupKey& InMaterialSettings)
	{
		return HashCombineFast(
			HashCombineFast(GetTypeHash(InMaterialSettings.Key), GetTypeHash(InMaterialSettings.Group)),
			GetTypeHash(InMaterialSettings.Tag)
		);
	}

	/** Defines what the material specs are */
	UPROPERTY()
	FText3DMaterialKey Key;

	/** Defines to which group this material is applied */
	EText3DGroupType Group = EText3DGroupType::Front;

	/** Defines a tag to differentiate materials by style */
	UPROPERTY()
	FName Tag = NAME_None;
};

USTRUCT()
struct FGlyphMeshParameters
{
	GENERATED_BODY()

	UPROPERTY()
	float Extrude = 5.0f;

	UPROPERTY()
	float Bevel = 0.0f;

	UPROPERTY()
	EText3DBevelType BevelType = EText3DBevelType::Convex;

	UPROPERTY()
	int32 BevelSegments = 8;

	UPROPERTY()
	bool bOutline = false;

	UPROPERTY()
	float OutlineExpand = 0.5f;

	UPROPERTY()
	EText3DOutlineType OutlineType = EText3DOutlineType::Stroke;

	UPROPERTY()
	FVector PivotOffset = FVector::ZeroVector;

	UPROPERTY()
	EText3DMeshType MeshType = EText3DMeshType::Unknown;

	friend uint32 GetTypeHash(const FGlyphMeshParameters& InElement)
	{
		uint32 HashParameters = 0;
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.Extrude));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.Bevel));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.BevelType));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.BevelSegments));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.bOutline));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.OutlineExpand));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.OutlineType));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.PivotOffset));
		HashParameters = HashCombine(HashParameters, GetTypeHash(InElement.MeshType));
		return HashParameters;
	}
};

USTRUCT()
struct FText3DCachedMesh
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	UPROPERTY()
	TObjectPtr<UDynamicMesh> DynamicMesh;

	UPROPERTY()
	FBox MeshBounds = FBox(ForceInitToZero);

	/** Pivot offset */
	UPROPERTY()
	FVector MeshOffset = FVector::ZeroVector;

	/** Used to scale mesh to achieve different font size */
	UPROPERTY()
	FVector2D FontFaceGlyphSize = FVector2D::ZeroVector;

	/** Amount of active usage of this glyph, used for cleanup */
	uint32 RefCount = 0;
};

USTRUCT()
struct FText3DMaterialOverride
{
	GENERATED_BODY()

	UPROPERTY()
	FName Tag = NAME_None;

	/** Materials where index = EText3DGroupType */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	bool operator==(const FText3DMaterialOverride& Other) const
	{
		return Tag == Other.Tag;
	}

	bool operator!=(const FText3DMaterialOverride& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FText3DMaterialOverride& InElement)
	{
		return GetTypeHash(InElement.Tag);
	}
};

namespace UE::Text3D
{
	namespace Priority
	{
		static constexpr uint16 Token = 0;
		static constexpr uint16 Geometry = 1;
		static constexpr uint16 Style = 2;
		static constexpr uint16 Layout = 3;
		static constexpr uint16 Effect = 4;
		static constexpr uint16 Material = 5;
		static constexpr uint16 Rendering = 6;
	}

	namespace Metrics
	{
		/** Default value to match size across different Text3D version */
		static constexpr float FontSize = 48.f;
		/** Scale used to transform freetype face result to get normalized values based on our font size */
		static constexpr float FontSizeInverse = 1.f / FontSize;
		/** Freetype conversion ratio to pixel */
		static constexpr float Convert26Dot6ToPixel = 1.f / 64.f;
	}

	namespace Renderer
	{
		struct FUpdateParameters
		{
			/** All dirty flags for this update */
			EText3DRendererFlags UpdateFlags = EText3DRendererFlags::None;

			/** Current flag being updated */
			EText3DRendererFlags CurrentFlag = EText3DRendererFlags::None;

			/** Whether CurrentFlag is the last flag of UpdateFlags */
			bool bIsLastFlag = false;
		};
	}

	namespace Geometry
	{
		/** Acts as a handle for FText3DCachedMesh */
		struct FCachedFontFaceGlyphHandle
		{
			FCachedFontFaceGlyphHandle() = default;
			FCachedFontFaceGlyphHandle(const FText3DFontFaceCache& InFontFaceCache, uint32 InGlyphIndex, const FGlyphMeshParameters& InGlyphMeshParameters);
			FCachedFontFaceGlyphHandle(const FCachedFontFaceGlyphHandle& InOther);
			FCachedFontFaceGlyphHandle(FCachedFontFaceGlyphHandle&& InOther);
			~FCachedFontFaceGlyphHandle();

			FCachedFontFaceGlyphHandle& operator=(const FCachedFontFaceGlyphHandle& InOther);
			FCachedFontFaceGlyphHandle& operator=(FCachedFontFaceGlyphHandle&& InOther);

			void Unset();

			const FText3DCachedMesh* Resolve() const;
			FText3DCachedMesh* Resolve();

			bool IsValid() const
			{
				return FontFaceHash != 0 && GlyphMeshHash != 0;
			}

			uint32 GetGlyphIndex() const
			{
				 return GlyphIndex;
			}

		private:
			uint32 FontFaceHash = 0;
			uint32 GlyphIndex = 0;
			uint32 GlyphMeshHash = 0;
		};
	}

	namespace Material
	{
		struct FMaterialParameters
		{
			/** Tag used to identify a material section, none = default */
			FName Tag = NAME_None;

			/** Group used to identify an entry in the material section */
			EText3DGroupType Group = EText3DGroupType::Front;
		};
	}
}