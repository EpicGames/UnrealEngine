// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Text3DTokenBase.generated.h"

/** Text3D token to swap variable by its content */
UCLASS(MinimalAPI, EditInlineNew, ClassGroup=Text3D, DisplayName="Token", AutoExpandCategories=(Token))
class UText3DTokenBase : public UObject
{
	GENERATED_BODY()

public:
	void SetTokenName(FName InName);
	FName GetTokenName() const
	{
		return TokenName;
	}

	void SetContent(const FText& InContent);
	const FText& GetContent() const
	{
		return Content;
	}

	/** Collects named tokens and their content for replacement */
	virtual void CollectTokens(FFormatNamedArguments& InNamedArguments);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnTokenPropertiesChanged();

	/** Token variable that will be swapped */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Token", meta=(AllowPrivateAccess="true"))
	FName TokenName = NAME_None;

	/** Content that replaces token in the text */
	UPROPERTY(EditAnywhere, Setter, Getter, Category="Token", meta=(AllowPrivateAccess="true"))
	FText Content;
};
