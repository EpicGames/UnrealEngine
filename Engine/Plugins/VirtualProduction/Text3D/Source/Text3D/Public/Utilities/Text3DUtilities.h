// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Text3DTypes.h"

class AStaticMeshActor;
class IText3DBuildSystemInterface;
class UFont;
class UFontFace;
class UText3DComponent;
struct FText3DTypeFaceMetrics;
struct FTypefaceEntry;

/** Utilities to manipulate text font */
namespace UE::Text3D::Utilities
{
	/** Finds the text build system for the given world context object */
	TEXT3D_API IText3DBuildSystemInterface* FindBuildSystem(const UObject* InWorldContextObject);

	namespace Font
	{
		/** Get proper font name without invalid chars */
		TEXT3D_API bool GetSanitizeFontName(const UFont* InFont, FString& OutFontName);

		/** Get font name based on object/import/legacy name */
		TEXT3D_API bool GetFontName(const UFont* InFont, FString& OutFontName);

		/** Replace invalid object name chars and spaces */
		TEXT3D_API void SanitizeFontName(FString& InOutFontName);

		/** Detect font style (italic, bold, monospaced) */
		TEXT3D_API bool GetFontStyle(const UFont* InFont, EText3DFontStyleFlags& OutFontStyleFlags);
	
		/** Detect font style (italic, bold, monospaced) */
		TEXT3D_API bool GetFontStyle(const FText3DFontFamily& InFontFamily, EText3DFontStyleFlags& OutFontStyleFlags);
	
		/** Get font faces that compose the font */
		TEXT3D_API bool GetFontFaces(const UFont* InFont, TArray<UFontFace*>& OutFontFaces);
	}

#if WITH_EDITOR
	namespace Conversion
	{
		/** Pick in which folder the asset object will be saved */
		bool PickAssetPath(const FString& InDefaultPath, FString& OutPickedPath);
		
		/** Converts the text component in a static mesh object and spawns a static mesh actor */
		AStaticMeshActor* ConvertToStaticMesh(const UText3DComponent* InComponent);
	}
#endif
}