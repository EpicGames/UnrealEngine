// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class Text3D : ModuleRules
{
	public Text3D(ReadOnlyTargetRules Target) : base(Target)
	{
		bEnableExceptions = true;
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[] 
		{
			"Core",
			"CoreUObject",
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(new string[] 
		{
			"DeveloperSettings",
			"GeometryAlgorithms",
			"GeometryCore",
			"GeometryFramework",
			"GeometryScriptingCore",
            "ICU",
            "MeshDescription",
            "Slate",
			"SlateCore",
			"StaticMeshDescription",
		});

		if (Target.Type != TargetType.Server)
		{
			if (Target.bCompileFreeType)
			{
				PrivateDependencyModuleNames.Add("FreeType2");
			}

			PrivateDependencyModuleNames.Add("HarfBuzz");
		}

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.Add("MeshMergeUtilities");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		// Needed to reference underlying FreeType info
		// @todo: Move font vector handling to new module ("FontCore") - don't expose FT directly?
		string EnginePath = Path.GetFullPath(Target.RelativeEnginePath);
		PrivateIncludePaths.AddRange(new string[]  {
			Path.Combine(EnginePath, "Source/Runtime/SlateCore/Private")
		});
	}
}
