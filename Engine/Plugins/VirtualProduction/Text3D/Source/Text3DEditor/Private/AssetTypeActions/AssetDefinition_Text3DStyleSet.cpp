// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetDefinition_Text3DStyleSet.h"

#include "Styles/Text3DStyleSet.h"

FText UAssetDefinition_Text3DStyleSet::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Text3DStyleSet", "Text3D Style Set");
}

FLinearColor UAssetDefinition_Text3DStyleSet::GetAssetColor() const
{
	return FLinearColor(FColor(128, 255, 128));
}

TSoftClassPtr<UObject> UAssetDefinition_Text3DStyleSet::GetAssetClass() const
{
	return UText3DStyleSet::StaticClass();
}