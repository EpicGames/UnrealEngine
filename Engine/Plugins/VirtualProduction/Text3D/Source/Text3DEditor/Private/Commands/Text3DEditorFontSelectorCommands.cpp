// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/Text3DEditorFontSelectorCommands.h"

#define LOCTEXT_NAMESPACE "Text3DEditorFontSelectorCommands"

FText3DEditorFontSelectorCommands::FText3DEditorFontSelectorCommands()
	: TCommands<FText3DEditorFontSelectorCommands>(UE_MODULE_NAME
	, LOCTEXT("FontSelectorCommandDesc", "Text3D Font Selector Commands")
	, NAME_None
	, FAppStyle::GetAppStyleSetName())
{
}

void FText3DEditorFontSelectorCommands::RegisterCommands()
{
	UI_COMMAND(ShowMonospacedFonts
		, "Monospaced"
		, "Filter monospaced fonts"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ShowBoldFonts
		, "Bold"
		, "Filter fonts with bold typefaces available"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ShowItalicFonts
		, "Italic"
		, "Filter fonts with italic typefaces available"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
