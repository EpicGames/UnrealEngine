// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FText3DEditorFontSelectorCommands : public TCommands<FText3DEditorFontSelectorCommands>
{
public:
	FText3DEditorFontSelectorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> ShowMonospacedFonts;
	TSharedPtr<FUICommandInfo> ShowBoldFonts;
	TSharedPtr<FUICommandInfo> ShowItalicFonts;
};
