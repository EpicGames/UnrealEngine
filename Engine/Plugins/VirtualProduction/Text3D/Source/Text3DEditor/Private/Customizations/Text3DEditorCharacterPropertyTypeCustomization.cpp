// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Text3DEditorCharacterPropertyTypeCustomization.h"

#include "Characters/Text3DCharacterBase.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Extensions/Text3DDefaultCharacterExtension.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Text3DComponent.h"
#include "UObject/Object.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text//STextBlock.h"

#define LOCTEXT_NAMESPACE "Text3DEditorCharacterPropertyTypeCustomization"

namespace UE::Text3DEditor::Customization
{
	TSharedRef<IPropertyTypeCustomization> FText3DEditorCharacterPropertyTypeCustomization::MakeInstance()
	{
		return MakeShared<FText3DEditorCharacterPropertyTypeCustomization>();
	}

	void FText3DEditorCharacterPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InUtils)
	{
		TArray<UObject*> Objects;
		InPropertyHandle->GetOuterObjects(Objects);
		if (Objects.Num() != 1)
		{
			return;
		}

		UText3DCharacterExtensionBase* CharacterExtension = Cast<UText3DCharacterExtensionBase>(Objects[0]);
		if (!CharacterExtension)
		{
			return;
		}

		PropertyUtilitiesWeak = InUtils.GetPropertyUtilities().ToWeakPtr();
		CharacterExtensionWeak = CharacterExtension;
		CharacterPropertyHandleWeak = InPropertyHandle;

		InPropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSPLambda(this, [this]()
		{
			if (const TSharedPtr<IPropertyHandle> PropertyHandle = CharacterPropertyHandleWeak.Pin())
			{
				PropertyHandle->SetValue(0);
				OnTextCharacterChanged();
				if (const TSharedPtr<IPropertyUtilities> Utilities = PropertyUtilitiesWeak.Pin())
				{
					Utilities->RequestForceRefresh();
				}
			}
		}));

		OnTextCharacterChanged();

		if (UText3DComponent* Text3DComponent = CharacterExtension->GetText3DComponent())
		{
			Text3DComponent->OnTextPostUpdate().AddSPLambda(this, [this](UText3DComponent* InComponent, EText3DRendererFlags InFlags)
			{
				if (EnumHasAnyFlags(InFlags, EText3DRendererFlags::Geometry))
				{
					const TOptional<uint16> MaxIndex = GetCharacterLastIndex();
					if (ActiveIndex > MaxIndex.Get(0))
					{
						OnTextCharacterChanged();

						if (const TSharedPtr<IPropertyHandle> ParentProperty = CharacterPropertyHandleWeak.Pin())
						{
							ParentProperty->RequestRebuildChildren();
						}
					}
				}
			});
		}

		InHeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.f, 0.f)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FText3DEditorCharacterPropertyTypeCustomization::GetCharacterText)
			]
		]
		.ValueContent()
		[
			SNew(SSpinBox<uint16>)
			.MinValue(0)
			.MaxValue(this, &FText3DEditorCharacterPropertyTypeCustomization::GetCharacterLastIndex)
			.Delta(1)
			.EnableSlider(true)
			.PreventThrottling(true)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value_Lambda([this]()
			{
				return ActiveIndex;
			})
			.OnValueChanged_Lambda([this](int32 InNewValue)
			{
				ActiveIndex = InNewValue;
			})
			.OnValueCommitted_Lambda([this](int32 InNewValue, ETextCommit::Type InType)
			{
				if (UText3DCharacterExtensionBase* Extension = CharacterExtensionWeak.Get())
				{
					if (InType != ETextCommit::Type::OnCleared)
					{
						ActiveIndex = InNewValue;
						Extension->TextCharacterIndex = ActiveIndex;

						if (const TSharedPtr<IPropertyUtilities> Utilities = PropertyUtilitiesWeak.Pin())
						{
							Utilities->RequestForceRefresh();
						}
					}
					else
					{
						ActiveIndex = Extension->TextCharacterIndex;
					}
				}
			})
		];
	}

	void FText3DEditorCharacterPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InUtils)
	{
		if (const UText3DCharacterExtensionBase* Extension = CharacterExtensionWeak.Get())
		{
			if (UText3DCharacterBase* Character = Extension->GetCharacter(Extension->TextCharacterIndex))
			{
				for (const FProperty* Property : TFieldRange<FProperty>(Character->GetClass()))
				{
					if (!Property || !Property->HasAnyPropertyFlags(CPF_Edit))
					{
						continue;
					}

					FAddPropertyParams AddParams;
					AddParams.CreateCategoryNodes(false);
					AddParams.HideRootObjectNode(true);

					if (const IDetailPropertyRow* Row = InChildBuilder.AddExternalObjectProperty({Character}, Property->GetFName(), AddParams))
					{
						if (const TSharedPtr<IPropertyHandle> RowHandle = Row->GetPropertyHandle())
						{
							// Due to AddExternalObjectProperty, EditCondition are not updating immediately so we give them a hand
							RowHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSPLambda(this, [this](const FPropertyChangedEvent& InEvent)
							{
								if (InEvent.ChangeType != EPropertyChangeType::Interactive)
								{
									if (const TSharedPtr<IPropertyHandle> ParentProperty = CharacterPropertyHandleWeak.Pin())
									{
										ParentProperty->RequestRebuildChildren();
									}
								}
							}));
						}
					}
				}
			}
		}
	}

	FText FText3DEditorCharacterPropertyTypeCustomization::GetCharacterText() const
	{
		if (const UText3DCharacterExtensionBase* Extension = CharacterExtensionWeak.Get())
		{
			return FText::Format(
				LOCTEXT("CharacterCount", "Total: {0}")
				, FText::AsNumber(Extension->GetCharacterCount())
			);
		}

		return FText::GetEmpty();
	}

	TOptional<uint16> FText3DEditorCharacterPropertyTypeCustomization::GetCharacterLastIndex() const
	{
		if (const UText3DCharacterExtensionBase* Extension = CharacterExtensionWeak.Get())
		{
			return FMath::Max(0, Extension->GetCharacterCount() - 1);
		}

		return 0;
	}

	void FText3DEditorCharacterPropertyTypeCustomization::OnTextCharacterChanged()
	{
		if (UText3DCharacterExtensionBase* Extension = CharacterExtensionWeak.Get())
		{
			ActiveIndex = FMath::Clamp(Extension->TextCharacterIndex, 0, GetCharacterLastIndex().Get(0));
			Extension->TextCharacterIndex = ActiveIndex;
		}
	}
}

#undef LOCTEXT_NAMESPACE
