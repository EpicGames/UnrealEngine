// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;
class UText3DCharacterExtensionBase;

namespace UE::Text3DEditor::Customization
{
	/** Only allow property customization with "TextCharacterSelector" metadata */
	class FText3DEditorCharacterPropertyTypeIdentifier : public IPropertyTypeIdentifier
	{
	public:
		//~ Begin IPropertyTypeIdentifier
		virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
		{
			return InPropertyHandle.HasMetaData(TEXT("TextCharacterSelector"));
		}
		//~ End IPropertyTypeIdentifier
	};

	/** Customization for UText3DCharacter object */
	class FText3DEditorCharacterPropertyTypeCustomization : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin IPropertyTypeCustomization
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
			, FDetailWidgetRow& InHeaderRow
			, IPropertyTypeCustomizationUtils& InUtils) override;

		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
			, IDetailChildrenBuilder& InChildBuilder
			, IPropertyTypeCustomizationUtils& InUtils) override;
		//~ End IPropertyTypeCustomization

	private:
		FText GetCharacterText() const;
		TOptional<uint16> GetCharacterLastIndex() const;
		void OnTextCharacterChanged();

		uint16 ActiveIndex = 0;
		TWeakPtr<IPropertyHandle> CharacterPropertyHandleWeak;
		TWeakObjectPtr<UText3DCharacterExtensionBase> CharacterExtensionWeak;
		TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
	};
}
