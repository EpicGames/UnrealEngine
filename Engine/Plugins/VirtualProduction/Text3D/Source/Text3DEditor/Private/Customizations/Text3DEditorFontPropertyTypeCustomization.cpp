// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Text3DEditorFontPropertyTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/Font.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Settings/Text3DProjectSettings.h"
#include "Subsystems/Text3DEditorFontSubsystem.h"
#include "Widgets/SText3DEditorFontField.h"
#include "Widgets/SText3DEditorFontSelector.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "Text3DEditorFontPropertyTypeCustomization"

TSharedRef<IPropertyTypeCustomization> FText3DEditorFontPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FText3DEditorFontPropertyTypeCustomization>();
}

void FText3DEditorFontPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
	, FDetailWidgetRow& InHeaderRow
	, IPropertyTypeCustomizationUtils& InUtils)
{
	FontPropertyHandle = InPropertyHandle;

	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SText3DEditorFontSelector, InPropertyHandle)
	];
}

void FText3DEditorFontPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
}

#undef LOCTEXT_NAMESPACE
