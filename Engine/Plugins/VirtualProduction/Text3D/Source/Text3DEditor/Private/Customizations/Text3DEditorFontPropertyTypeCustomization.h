// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;

/** Only allow property customization with "AdvancedFontPicker" metadata */
class FText3DEditorFontPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	//~ Begin IPropertyTypeIdentifier
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		return InPropertyHandle.HasMetaData(TEXT("AdvancedFontPicker"));
	}
	//~ End IPropertyTypeIdentifier
};

/** Customization for UFont object to display an advanced font picker for project and system fonts */
class FText3DEditorFontPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
		, FDetailWidgetRow& InHeaderRow
		, IPropertyTypeCustomizationUtils& InUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
		, IDetailChildrenBuilder& InChildBuilder
		, IPropertyTypeCustomizationUtils& InUtils) override;
	//~ End IPropertyTypeCustomization

private:
	TSharedPtr<IPropertyHandle> FontPropertyHandle;
};
