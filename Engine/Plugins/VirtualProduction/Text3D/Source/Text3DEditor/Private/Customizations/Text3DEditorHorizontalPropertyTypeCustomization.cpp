// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Text3DEditorHorizontalPropertyTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Widgets/SText3DEditorHorizontalAlignment.h"

TSharedRef<IPropertyTypeCustomization> FText3DEditorHorizontalPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FText3DEditorHorizontalPropertyTypeCustomization>();
}

void FText3DEditorHorizontalPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	InRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SText3DEditorHorizontalAlignment, InPropertyHandle)
	];
}

void FText3DEditorHorizontalPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
	
}
