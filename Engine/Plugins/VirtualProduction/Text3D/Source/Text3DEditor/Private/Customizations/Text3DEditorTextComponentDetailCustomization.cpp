// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Text3DEditorTextComponentDetailCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Renderers/Text3DRendererBase.h"
#include "Text3DComponent.h"
#include "UObject/Class.h"
#include "Widgets/Input/SButton.h"

namespace UE::Text3DEditor::Customization
{
	void FText3DEditorTextComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		const FName ComponentClassName = UText3DComponent::StaticClass()->GetFName();

		const TArray<FName> CategoryNames =
		{
			TEXT("Text"),
			TEXT("Token"),
			TEXT("Style"),
			TEXT("Character"),
			TEXT("Layout"),
			TEXT("LayoutEffects"),
			TEXT("Geometry"),
			TEXT("Material"),
			TEXT("Rendering"),
			TEXT("Utilities")
		};

		int32 SortOrder = 1100;
		for (FName CategoryName : CategoryNames)
		{
			InDetailBuilder.EditCategory(CategoryName).SetSortOrder(SortOrder);
			SortOrder += 100;
		}

		static bool bSectionInitialized = false;
		if (!bSectionInitialized)
		{
			bSectionInitialized = true;

			const FName GeometrySectionName = TEXT("Geometry");
			const TSharedRef<FPropertySection> GeometrySection = PropertyModule.FindOrCreateSection(ComponentClassName, GeometrySectionName, FText::FromName(GeometrySectionName));
			GeometrySection->AddCategory(TEXT("Geometry"));

			const FName LayoutSectionName = TEXT("Layout");
			const TSharedRef<FPropertySection> LayoutSection = PropertyModule.FindOrCreateSection(ComponentClassName, LayoutSectionName, FText::FromName(LayoutSectionName));
			LayoutSection->AddCategory(TEXT("Layout"));
			LayoutSection->AddCategory(TEXT("LayoutEffects"));
			LayoutSection->AddCategory(TEXT("Character"));

			const FName RenderingSectionName = TEXT("Rendering");
			const TSharedRef<FPropertySection> RenderingSection = PropertyModule.FindOrCreateSection(ComponentClassName, RenderingSectionName, FText::FromName(RenderingSectionName));
			RenderingSection->AddCategory(TEXT("Rendering"));

			const FName TextSectionName = TEXT("Text");
			const TSharedRef<FPropertySection> TextSection = PropertyModule.FindOrCreateSection(ComponentClassName, TextSectionName, FText::FromName(TextSectionName));
			TextSection->AddCategory(TEXT("Text"));
			TextSection->AddCategory(TEXT("Token"));

			const FName StyleSectionName = TEXT("Style");
			const TSharedRef<FPropertySection> MaterialSection = PropertyModule.FindOrCreateSection(ComponentClassName, StyleSectionName, FText::FromName(StyleSectionName));
			MaterialSection->AddCategory(TEXT("Style"));
			MaterialSection->AddCategory(TEXT("Material"));

			const FName EffectSectionName = TEXT("Effects");
			const TSharedRef<FPropertySection> EffectsSection = PropertyModule.FindOrCreateSection(ComponentClassName, EffectSectionName, FText::FromName(EffectSectionName));
			EffectsSection->AddCategory(TEXT("LayoutEffects"));

			const FName UtilitiesSectionName = TEXT("Utilities");
			const TSharedRef<FPropertySection> UtilitiesSection = PropertyModule.FindOrCreateSection(ComponentClassName, UtilitiesSectionName, FText::FromName(UtilitiesSectionName));
		}

		TArray<TWeakObjectPtr<UText3DComponent>> TextComponentsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UText3DComponent>();

		// Handle ufunctions
		TMap<FName, FName> FunctionToCategory;

		auto AddUFunction = [this, &FunctionToCategory](UObject* InContext, UFunction* InFunction)
		{
			// Only CallInEditor function with 0 parameters
			if (InFunction
				&& !InFunction->HasMetaData(TEXT("DeprecatedFunction"))
				&& InFunction->HasMetaData("CallInEditor")
				&& InFunction->NumParms == 0)
			{
				const FName FunctionName = InFunction->GetFName();
				TArray<FObjectFunction>& ObjectFunctions = NamedObjectFunctions.FindOrAdd(FunctionName);
				ObjectFunctions.Add(
					{
						.Owner = InContext,
						.Function = InFunction
					}
				);
				FunctionToCategory.Add(FunctionName, FName(InFunction->GetMetaData(TEXT("Category"))));
			}
		};

		for (const TWeakObjectPtr<UText3DComponent>& TextComponentWeak : TextComponentsWeak)
		{
			UText3DComponent* TextComponent = TextComponentWeak.Get();

			if (!TextComponent)
			{
				continue;
			}

			// Look for ufunction in component
			for (UFunction* Function : TFieldRange<UFunction>(TextComponent->GetClass(), EFieldIteratorFlags::ExcludeSuper))
			{
				AddUFunction(TextComponent, Function);
			}

			// Look for ufunction in renderer
			if (UText3DRendererBase* Renderer = TextComponent->GetTextRenderer())
			{
				for (UFunction* Function : TFieldRange<UFunction>(Renderer->GetClass(), EFieldIteratorFlags::ExcludeSuper))
				{
					AddUFunction(Renderer, Function);
				}
			}
		}

		if (!FunctionToCategory.IsEmpty())
		{
			TMap<FName, const TSharedPtr<SVerticalBox>> FunctionToWidget;

			// Add buttons for ufunctions based on their category
			for (const TPair<FName, FName>& FunctionToCategoryPair : FunctionToCategory)
			{
				const FName FunctionsCategoryName = FunctionToCategoryPair.Value;

				IDetailCategoryBuilder& FunctionsCategory = InDetailBuilder.EditCategory(FunctionsCategoryName, FText::FromName(FunctionsCategoryName), ECategoryPriority::Uncommon);

				// Subcategories are not supported in sections
				if (!FunctionsCategoryName.ToString().Contains(TEXT("|")))
				{
					const TSharedRef<FPropertySection> FunctionsSection = PropertyModule.FindOrCreateSection(ComponentClassName, FunctionsCategoryName, FText::FromName(FunctionsCategoryName));
					FunctionsSection->AddCategory(FunctionsCategoryName);
				}

				const TSharedPtr<SVerticalBox> FunctionsWidget = SNew(SVerticalBox);

				FunctionsCategory.AddCustomRow(FText::GetEmpty())
					.WholeRowContent()
					.HAlign(HAlign_Left)
					[
						FunctionsWidget.ToSharedRef()
					];

				FunctionToWidget.Add(FunctionToCategoryPair.Key, FunctionsWidget);
			}

			for (const TPair<FName, const TSharedPtr<SVerticalBox>>& FunctionNameToWidget : FunctionToWidget)
			{
				const FText ButtonLabel = FText::FromString(FName::NameToDisplayString(FunctionNameToWidget.Key.ToString(), /** IsBool */false));

				const TSharedPtr<SVerticalBox>& FunctionsWidget = FunctionNameToWidget.Value;

				FunctionsWidget->AddSlot()
					.Padding(0.f, 3.f)
					.AutoHeight()
					[
						SNew(SButton)
						.Text(ButtonLabel)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						.OnClicked(this, &FText3DEditorTextComponentDetailCustomization::OnFunctionButtonClicked, FunctionNameToWidget.Key)
					];
			}
		}
	}

	FReply FText3DEditorTextComponentDetailCustomization::OnFunctionButtonClicked(FName InFunctionName)
	{
		if (TArray<FObjectFunction> const* ObjectFunctions = NamedObjectFunctions.Find(InFunctionName))
		{
			for (const FObjectFunction& ObjectFunction : *ObjectFunctions)
			{
				UObject* Object = ObjectFunction.Owner.Get();
				UFunction* Function = ObjectFunction.Function.Get();

				if (!Object || !Function)
				{
					continue;
				}

				Object->ProcessEvent(Function, nullptr);
			}
		}

		return FReply::Handled();
	}
}
