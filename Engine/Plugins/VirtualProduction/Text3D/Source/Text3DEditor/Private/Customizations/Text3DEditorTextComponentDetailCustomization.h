// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class IPropertyHandle;
class IPropertyUtilities;
class UFunction;
class UObject;

namespace UE::Text3DEditor::Customization
{
	/** Used to customize Text3D component properties in details panel */
	class FText3DEditorTextComponentDetailCustomization : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShared<FText3DEditorTextComponentDetailCustomization>();
		}

		//~ Begin IDetailCustomization
		virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
		//~ End IDetailCustomization

	protected:
		struct FObjectFunction
		{
			TWeakObjectPtr<UObject> Owner;
			TWeakObjectPtr<UFunction> Function;
		};

		/** Execute ufunction with that name on selected objects */
		FReply OnFunctionButtonClicked(FName InFunctionName);

		/** Function name to object ufunction mapping */
		TMap<FName, TArray<FObjectFunction>> NamedObjectFunctions;
	};
}