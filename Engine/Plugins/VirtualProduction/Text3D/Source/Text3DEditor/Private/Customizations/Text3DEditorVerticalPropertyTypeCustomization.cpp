// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Text3DEditorVerticalPropertyTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "Widgets/SText3DEditorVerticalAlignment.h"

TSharedRef<IPropertyTypeCustomization> FText3DEditorVerticalPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FText3DEditorVerticalPropertyTypeCustomization>();
}

void FText3DEditorVerticalPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils)
{
	InRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SText3DEditorVerticalAlignment, InPropertyHandle)
	];
}

void FText3DEditorVerticalPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
{
	
}
