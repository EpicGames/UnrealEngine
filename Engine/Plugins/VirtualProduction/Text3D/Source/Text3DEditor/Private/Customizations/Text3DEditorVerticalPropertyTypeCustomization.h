// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;

/** Only allow property customization with "CustomAlignmentWidget" metadata */
class FText3DEditorVerticalPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	//~ Begin IPropertyTypeIdentifier
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		return InPropertyHandle.HasMetaData(TEXT("CustomAlignmentWidget"));
	}
	//~ End IPropertyTypeIdentifier
};

/** Used to customize EText3DVerticalTextAlignment enum */
class FText3DEditorVerticalPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// ~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InRow, IPropertyTypeCustomizationUtils& InUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
	// ~ End IPropertyTypeCustomization interface

protected:
	TSharedPtr<IPropertyHandle> PropertyHandle;
};