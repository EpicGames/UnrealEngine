// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/Text3DEditorActorFactory.h"
#include "Text3DActor.h"

#define LOCTEXT_NAMESPACE "Text3DEditorActorFactory"

UText3DEditorActorFactory::UText3DEditorActorFactory()
{
	DisplayName = LOCTEXT("Text3D", "Text 3D");
	NewActorClass = AText3DActor::StaticClass();
	SpawnPositionOffset = FVector(0, 0, 0);
	bUseSurfaceOrientation = false;
}

#undef LOCTEXT_NAMESPACE
