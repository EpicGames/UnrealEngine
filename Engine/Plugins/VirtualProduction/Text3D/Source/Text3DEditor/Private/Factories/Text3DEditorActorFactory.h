// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "Text3DEditorActorFactory.generated.h"

UCLASS()
class UText3DEditorActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UText3DEditorActorFactory();
};
