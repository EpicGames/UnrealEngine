// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/Text3DEditorStyleSetFactory.h"

#include "Styles/Text3DStyleSet.h"

UText3DEditorStyleSetFactory::UText3DEditorStyleSetFactory()
{
	SupportedClass = UText3DStyleSet::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UText3DEditorStyleSetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (!ensure(SupportedClass == InClass))
	{
		return nullptr;
	}

	return NewObject<UText3DStyleSet>(InParent, InName, InFlags);
}
