// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Text3DEditorStyleSetFactory.generated.h"

/** Factory that creates a Style Set asset */
UCLASS()
class UText3DEditorStyleSetFactory : public UFactory
{
	GENERATED_BODY()

	UText3DEditorStyleSetFactory();

	//~ Begin UFactory
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	//~ End UFactory
};
