// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxSystemFontLoading.h"

#include "Text3DTypes.h"

namespace UE::Text3D::Private::Fonts
{
	void GetSystemFontInfo(TMap<FString, FText3DFontFamily>& OutFontsInfo)
	{
	}

	void ListAvailableFontFiles()
	{
	}
}
