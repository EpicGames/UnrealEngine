// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacSystemFontLoading.h"
#include "Mac/MacSystemIncludes.h"
#include <ApplicationServices/ApplicationServices.h>
#include <CoreText/CoreText.h>
#include <Foundation/Foundation.h>
#include "Logs/Text3DEditorLogs.h"
#include "Misc/Paths.h"
#include "Text3DTypes.h"

namespace UE::Text3D::Private::Fonts
{
	void GetSystemFontInfo(TMap<FString, FText3DFontFamily>& OutFontsInfo)
	{
		// Cache font collection for fast use in future calls
		static CTFontCollectionRef Collection = nullptr;
		if (!Collection)
		{
			Collection = CTFontCollectionCreateFromAvailableFonts(nullptr);
		}

		CFArrayRef FontDescriptors = CTFontCollectionCreateMatchingFontDescriptors(Collection);
		CFIndex FontsCount = CFArrayGetCount(FontDescriptors);
		for (CFIndex CurrentFontIndex = 0; CurrentFontIndex < FontsCount; CurrentFontIndex++)
		{
			CTFontDescriptorRef FontDescriptorRef = (CTFontDescriptorRef) CFArrayGetValueAtIndex(FontDescriptors, CurrentFontIndex);
			
			FString FontPath;
			if (NSURL* URLAttribute = (NSURL*) CTFontDescriptorCopyAttribute(FontDescriptorRef, kCTFontURLAttribute))
			{
				if (NSString* URLString = URLAttribute.absoluteString)
				{
					FontPath = FString(URLString.UTF8String);
				}
				
				CFRelease(URLAttribute);
			}
			
			if (FontPath.IsEmpty())
			{
				continue;
			}

			FString FontFaceName;
			if (NSString* FontFaceNameAttribute = (NSString*) CTFontDescriptorCopyAttribute(FontDescriptorRef, kCTFontStyleNameAttribute))
			{
				FontFaceName = FString(FontFaceNameAttribute.UTF8String);
				CFRelease(FontFaceNameAttribute);
			}
			else
			{
				continue;
			}

			FString FontFamilyName;
			if (NSString* FontFamilyNameAttribute = (NSString*) CTFontDescriptorCopyAttribute(FontDescriptorRef, kCTFontFamilyNameAttribute))
			{
				FontFamilyName = FString(FontFamilyNameAttribute.UTF8String);
				CFRelease(FontFamilyNameAttribute);
			}
			else
			{
				continue;
			}
		
			FontFaceName.RemoveFromStart(FontFamilyName);
			FontFaceName.RemoveFromStart(TEXT(" "));

			if (FontFaceName.IsEmpty())
			{
				FontFaceName = TEXT("Regular");
			}	

			// Clean the url to properly find the file
			FontPath.RemoveFromStart("file://");
			FontPath.ReplaceInline(TEXT("%20"), TEXT(" "));

			if (FPaths::FileExists(FontPath))
			{
				FText3DFontFamily& FontRetrieveParams = OutFontsInfo.FindOrAdd(FontFamilyName);
				FontRetrieveParams.FontFamilyName = FontFamilyName;
				FontRetrieveParams.AddFontFace(FontFaceName, FontPath);
			}
		}
		
		CFRelease(FontDescriptors);
	}

	void ListAvailableFontFiles()
	{
		TMap<FString, FText3DFontFamily> FontsInfoMap;
		GetSystemFontInfo(FontsInfoMap);

		if (FontsInfoMap.IsEmpty())
		{
			return;
		}

		UE_LOGF(LogText3DEditor, Log, "Font Manager Subsystem: listing system fonts and their typefaces:");
		for (const TPair<FString, FText3DFontFamily>& FontsInfoPair : FontsInfoMap)
		{
			const FText3DFontFamily& FontParameters = FontsInfoPair.Value;
			UE_LOGF(LogText3DEditor, Log, "== Font: %ls ==", *FontParameters.FontFamilyName);

			int32 FontFaceIndex = 0;
			for (const TPair<FString, FString>& FontFaceName : FontParameters.FontFacePaths)
			{
				UE_LOGF(LogText3DEditor, Log, "\t\tFace Name: %ls found at %ls", *FontFaceName.Key, *FontFaceName.Value);
				FontFaceIndex++;
			}
		}
	}
}
