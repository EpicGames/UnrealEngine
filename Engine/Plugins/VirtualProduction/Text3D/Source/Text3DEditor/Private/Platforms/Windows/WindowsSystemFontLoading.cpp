// Copyright Epic Games, Inc. All Rights Reserved.

//#pragma warning(disable: 6385)

#include "WindowsSystemFontLoading.h"

#include "Text3DTypes.h"
#include "Logs/Text3DEditorLogs.h"

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "DWrite.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

namespace UE::Text3D::Private::Fonts
{
	HRESULT GetLocalizedName(IDWriteLocalizedStrings* InLocalizedStrings, FString& OutLocalizedName)
	{
		uint32 NameIndex = 0;
		uint32 NameLength = 0;
		BOOL NameExists = false;

		InLocalizedStrings->FindLocaleName(L"en-us", &NameIndex, &NameExists);

		// If the name still doesn't exist just take the first one.
		if (!NameExists)
		{
			NameIndex = 0;
		}

		HRESULT Result = InLocalizedStrings->GetStringLength(NameIndex, &NameLength);
		if (FAILED(Result))
		{
			return Result;
		}

		WCHAR* LocalizedString = new WCHAR[NameLength + 1];
		Result = InLocalizedStrings->GetString(NameIndex, LocalizedString, NameLength + 1);
		if (FAILED(Result))
		{
			return Result;
		}

		OutLocalizedName = LocalizedString;

		return Result;
	}

	void GetSystemFontInfo(TMap<FString, FText3DFontFamily>& OutFontsInfoMap)
	{
		// note: IDWriteFactory has newer versions, e.g. up to IDWriteFactory7, which apparently should provide better Font Families enumeration and naming.
		// Unfortunately, those are not included in UE's ThirdParty/DWrite version.
		// The system API (dwrite_3.h) provides it, but the declaration is guarded by #if NTDDI_VERSION >= NTDDI_WIN10_RS4, which is currently false.

		IDWriteFactory* DirectWriteFactory = nullptr;
		HRESULT Result = DWriteCreateFactory(DWRITE_FACTORY_TYPE_SHARED, __uuidof(IDWriteFactory), reinterpret_cast<IUnknown**>(&DirectWriteFactory));
		if (FAILED(Result))
		{
			return;
		}

		IDWriteFontCollection* FontCollection = nullptr;
		Result = DirectWriteFactory->GetSystemFontCollection(&FontCollection);
		if (FAILED(Result))
		{
			return;
		}

		// We are not adding font faces sharing the same path. This might change in the future.
		// It looks like .TTF fonts are not used with multiple typefaces in the same file, but Windows is picking multiple typefaces.
		// We can maybe improve on this, but for the moment being we just avoid having the same family + faces multiple times with different names
		TArray<FString> UsedPaths;

		const uint32 FamiliesCount = FontCollection->GetFontFamilyCount();
		for (uint32 FontFamilyIndex = 0; FontFamilyIndex < FamiliesCount; FontFamilyIndex++)
		{
			IDWriteFontFamily* FontFamily = nullptr;
			Result = FontCollection->GetFontFamily(FontFamilyIndex, &FontFamily);
			if (FAILED(Result))
			{
				continue;
			}

			// Get a list of localized strings for the family name.
			IDWriteLocalizedStrings* FamilyNames = nullptr;
			Result = FontFamily->GetFamilyNames(&FamilyNames);
			if (FAILED(Result))
			{
				continue;
			}

			// Get the family name.
			FString FamilyName;
			Result = GetLocalizedName(FamilyNames, FamilyName);
			if (FAILED(Result))
			{
				continue;
			}

			FText3DFontFamily FontInfo;

			const uint32 FontsCount = FontFamily->GetFontCount();
			for (uint32 FontIndex = 0; FontIndex < FontsCount; FontIndex++)
			{
				IDWriteFont* Font = nullptr;
				Result = FontFamily->GetFont(FontIndex, &Font);
				if (FAILED(Result))
				{
					continue;
				}

				IDWriteLocalizedStrings* FaceNames = nullptr;
				Result = Font->GetFaceNames(&FaceNames);
				if (FAILED(Result))
				{
					continue;
				}

				FString FaceName;
				Result = GetLocalizedName(FaceNames, FaceName);
				if (FAILED(Result))
				{
					continue;
				}

				IDWriteFontFace* FontFace = nullptr;
				Font->CreateFontFace(&FontFace);

				uint32 FilesCount = 0;
				Result = FontFace->GetFiles(&FilesCount, nullptr);
				if (FAILED(Result))
				{
					FontFace->Release();
					continue;
				}

				const uint32 FilesNum = FilesCount;
				IDWriteFontFile** FontFiles = new IDWriteFontFile*[FilesNum];

				Result = FontFace->GetFiles(&FilesCount, FontFiles);
				if (FAILED(Result))
				{
					delete[] FontFiles;
					FontFace->Release();
					continue;
				}

				for (uint32 FileIndex = 0; FileIndex < FilesNum; FileIndex++)
				{
					LPCVOID FontFileReferenceKey;
					uint32 FontFileReferenceKeySize;

					IDWriteFontFile* FontFile = FontFiles[FileIndex];
					if (!FontFile)
					{
						continue;
					}

					Result = FontFile->GetReferenceKey(&FontFileReferenceKey, &FontFileReferenceKeySize);
					if (FAILED(Result))
					{
						FontFile->Release();
						continue;
					}

					IDWriteFontFileLoader* FontFileLoader;
					Result = FontFile->GetLoader(&FontFileLoader);
					if (FAILED(Result))
					{
						FontFile->Release();
						continue;
					}

					IDWriteLocalFontFileLoader* LocalFontFileLoader;
					Result = FontFileLoader->QueryInterface(__uuidof(IDWriteLocalFontFileLoader), (void**)&LocalFontFileLoader);
					if (FAILED(Result))
					{
						FontFileLoader->Release();
						FontFile->Release();
						continue;
					}

					uint32 PathLength;
					Result = LocalFontFileLoader->GetFilePathLengthFromKey(FontFileReferenceKey, FontFileReferenceKeySize, &PathLength);
					if (FAILED(Result))
					{
						LocalFontFileLoader->Release();
						FontFileLoader->Release();
						FontFile->Release();
						continue;
					}

					WCHAR* FilePath = new WCHAR[PathLength + 1];
					Result = LocalFontFileLoader->GetFilePathFromKey(FontFileReferenceKey, FontFileReferenceKeySize, FilePath, PathLength + 1);
					if (FAILED(Result))
					{
						LocalFontFileLoader->Release();
						FontFileLoader->Release();
						FontFile->Release();
						continue;
					}

					const FString& Path = FilePath;
					if (UsedPaths.Contains(Path))
					{
						LocalFontFileLoader->Release();
						FontFileLoader->Release();
						FontFile->Release();
						continue;
					}

					UsedPaths.Add(Path);

					FontInfo.FontFamilyName = FamilyName;
					FontInfo.AddFontFace(FaceName, Path);

					LocalFontFileLoader->Release();
					FontFileLoader->Release();
					FontFile->Release();
				}

				delete[] FontFiles;
				FontFace->Release();
			}

			if (FontInfo.FontFacePaths.IsEmpty())
			{
				continue;
			}

			OutFontsInfoMap.Add(FamilyName, FontInfo);
		}
	}

	void ListAvailableFontFiles()
	{
		TMap<FString, FText3DFontFamily> FontsInfoMap;
		GetSystemFontInfo(FontsInfoMap);

		if (FontsInfoMap.IsEmpty())
		{
			return;
		}

		UE_LOGF(LogText3DEditor, Log, "Font Manager Subsystem: listing system fonts and their typefaces:");

		for (const TPair<FString, FText3DFontFamily>& FontsInfoPair : FontsInfoMap)
		{
			const FText3DFontFamily& FontParameters = FontsInfoPair.Value;
			UE_LOGF(LogText3DEditor, Log, "== Font: %ls ==", *FontParameters.FontFamilyName);

			int32 FontFaceIndex = 0;
			for (const TPair<FString, FString>& FontFaceName : FontParameters.FontFacePaths)
			{
				UE_LOGF(LogText3DEditor, Log, "\t\tFace Name: %ls found at %ls", *FontFaceName.Key, *FontFaceName.Value);
				FontFaceIndex++;
			}
		}
	}
}
