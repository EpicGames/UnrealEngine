// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"

struct FText3DFontFamily;

namespace UE::Text3D::Private::Fonts
{
	void GetSystemFontInfo(TMap<FString, FText3DFontFamily>& OutFontsInfo);
	void ListAvailableFontFiles();
}
