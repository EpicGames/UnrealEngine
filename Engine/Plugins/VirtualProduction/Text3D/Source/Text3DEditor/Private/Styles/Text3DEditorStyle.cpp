// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/Text3DEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FText3DEditorStyle::FText3DEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon16x16(16.f, 16.f);
	
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	// ClassIcon
	Set("ClassIcon.Text3DActor", new IMAGE_BRUSH("Text3DActor_16x", Icon16x16));
	Set("ClassThumbnail.Text3DActor", new IMAGE_BRUSH("Text3DActor_64x", FVector2D (64.0f, 64.0f)));

	// Alignments
	Set("Icons.Alignment.Left", new IMAGE_BRUSH_SVG("AlignLeft", Icon16x16));
	Set("Icons.Alignment.Center_Y", new IMAGE_BRUSH_SVG("AlignCenterHoriz", Icon16x16));
	Set("Icons.Alignment.Right", new IMAGE_BRUSH_SVG("AlignRight", Icon16x16));
	Set("Icons.Alignment.Top", new IMAGE_BRUSH_SVG("AlignTop", Icon16x16));
	Set("Icons.Alignment.Center_Z", new IMAGE_BRUSH_SVG("AlignCenterVert", Icon16x16));
	Set("Icons.Alignment.Bottom", new IMAGE_BRUSH_SVG("AlignBottom", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FText3DEditorStyle::~FText3DEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
