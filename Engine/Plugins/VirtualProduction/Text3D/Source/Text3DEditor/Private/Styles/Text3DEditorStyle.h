// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FText3DEditorStyle : public FSlateStyleSet
{
public:
	FText3DEditorStyle();
	
	virtual ~FText3DEditorStyle() override;
	
	static FText3DEditorStyle& Get()
	{
		static FText3DEditorStyle StyleSet;
		return StyleSet;
	}
};