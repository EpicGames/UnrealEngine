// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/Text3DEditorFontSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AutomatedAssetImportData.h"
#include "Editor.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Factories/FontFileImportFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformFileManager.h"
#include "Logs/Text3DEditorLogs.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "PackageTools.h"
#include "Platforms/PlatformSystemFontLoading.h"
#include "Settings/Text3DProjectSettings.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/SavePackage.h"
#include "Utilities/Text3DUtilities.h"
#include "Text3DComponent.h"

UText3DEditorFontSubsystem* UText3DEditorFontSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UText3DEditorFontSubsystem>();
	}

	return nullptr;
}

bool UText3DEditorFontSubsystem::IsFontFileSupported(const FString& InFontFilePath)
{
	const FString FileName = FPaths::GetCleanFilename(InFontFilePath);
	const FString Extension = FPaths::GetExtension(FileName).ToLower();
	return Extension == TEXT("ttf") || Extension == TEXT("otf");
}

bool UText3DEditorFontSubsystem::ImportSystemFont(const FString& InFontName)
{
	const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get();
	if (!Text3DSettings)
	{
		return false;
	}

	const FText3DEditorFont* EditorFont = SystemFonts.Find(InFontName);
	if (!EditorFont)
	{
		return false;
	}

	if (IsProjectFontUpToDate(InFontName))
	{
		return false;
	}

	if (!IsValid(EditorFont->Font))
	{
		return false;
	}

	if (EditorFont->Font->IsAsset())
	{
		return false;
	}

	const FString FontPackageName = TEXT("/") + Text3DSettings->GetFontDirectory().TrimChar('/') + TEXT("/Fonts/") + InFontName;
	UPackage* FontPackage = CreatePackage(*FontPackageName);

	if (!IsValid(FontPackage))
	{
		return false;
	}

	if (!UPackage::IsEmptyPackage(FontPackage))
	{
		return false;
	}

	const FString FontAssetFileName = FPackageName::LongPackageNameToFilename(FontPackage->GetPathName(), FPackageName::GetAssetPackageExtension());

	FontPackage->AddToRoot();
	FontPackage->FullyLoad();

	EditorFont->Font->Rename(nullptr, FontPackage, REN_NonTransactional | REN_DoNotDirty);
	FontPackage->MarkPackageDirty();
	PackagesToSave.Add(FontPackage);

	for (UFontFace* FontFace: EditorFont->FontFaces)
	{
		if (FontFace)
		{
			const FString FontFacePackageName = TEXT("/") + Text3DSettings->GetFontDirectory().TrimChar('/') + TEXT("/FontFaces/") + FontFace->GetName();
			if (UPackage* FontFacePackage = CreatePackage(*FontFacePackageName))
			{
				FontFace->Rename(nullptr, FontFacePackage, REN_NonTransactional | REN_DoNotDirty);
				FontFacePackage->MarkPackageDirty();
				FAssetRegistryModule::AssetCreated(FontFace);
				PackagesToSave.Add(FontFacePackage);
			}
		}
	}

	FAssetRegistryModule::AssetCreated(EditorFont->Font);

	RegisterProjectFont(EditorFont->Font);

	return true;
}

TArray<FString> UText3DEditorFontSubsystem::GetProjectFontNames() const
{
	TArray<FString> FontNames;
	ProjectFonts.GenerateKeyArray(FontNames);
	return FontNames;
}

TArray<FString> UText3DEditorFontSubsystem::GetSystemFontNames() const
{
	TArray<FString> FontNames;
	SystemFonts.GenerateKeyArray(FontNames);
	return FontNames;
}

TArray<FString> UText3DEditorFontSubsystem::GetFavoriteFontNames() const
{
	TArray<FString> FavoriteFontNames;

	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		for (const FString& FavoriteFontName : Text3DSettings->GetFavoriteFonts())
		{
			if (!!GetEditorFont(FavoriteFontName))
			{
				FavoriteFontNames.Emplace(FavoriteFontName);
			}
		}
	}

	return FavoriteFontNames;
}

const FText3DEditorFont* UText3DEditorFontSubsystem::GetEditorFont(const FString& InFontName) const
{
	const FText3DEditorFont* EditorFont = GetProjectFont(InFontName);

	if (!EditorFont)
	{
		EditorFont = GetSystemFont(InFontName);
	}

	return EditorFont;
}

const FText3DEditorFont* UText3DEditorFontSubsystem::GetSystemFont(const FString& InFontName) const
{
	FString SanitizedFontName = InFontName;
	UE::Text3D::Utilities::Font::SanitizeFontName(SanitizedFontName);
	return SystemFonts.Find(SanitizedFontName);
}

const FText3DEditorFont* UText3DEditorFontSubsystem::GetProjectFont(const FString& InFontName) const
{
	FString SanitizedFontName = InFontName;
	UE::Text3D::Utilities::Font::SanitizeFontName(SanitizedFontName);
	return ProjectFonts.Find(SanitizedFontName);
}

const FText3DEditorFont* UText3DEditorFontSubsystem::FindEditorFont(const UFont* InFont) const
{
	if (!IsValid(InFont))
	{
		return nullptr;
	}

	FString SanitizedFontName;
	if (!UE::Text3D::Utilities::Font::GetSanitizeFontName(InFont, SanitizedFontName))
	{
		UE_LOGF(LogText3DEditor, Warning, "Could not retrieve sanitized font name %ls", *InFont->GetName());
		return nullptr;
	}

	return GetEditorFont(SanitizedFontName);
}

void UText3DEditorFontSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	if (AssetRegistryModule.IsValid())
	{
		AssetRegistryModule.Get().OnFilesLoaded().AddUObject(this, &UText3DEditorFontSubsystem::OnAssetLoaded);
		AssetRegistryModule.Get().OnAssetAdded().AddUObject(this, &UText3DEditorFontSubsystem::OnAssetAdded);
		AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddUObject(this, &UText3DEditorFontSubsystem::OnAssetDeleted);
	}

	FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &UText3DEditorFontSubsystem::OnSaveImportedFonts);

	UText3DComponent::OnResolveFontByNameDelegate.BindUObject(this, &UText3DEditorFontSubsystem::ResolveFontByName);
}

void UText3DEditorFontSubsystem::Deinitialize()
{
	Super::Deinitialize();

	FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry"));

	if (AssetRegistryModule && AssetRegistryModule->IsValid())
	{
		AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(this);
		AssetRegistryModule->Get().OnAssetAdded().RemoveAll(this);
		AssetRegistryModule->Get().OnInMemoryAssetDeleted().RemoveAll(this);
	}

	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	
	UText3DComponent::OnResolveFontByNameDelegate.Unbind();
}

bool UText3DEditorFontSubsystem::IsProjectFontUpToDate(const FString& InFontName)
{
	const FText3DEditorFont* SystemFont = SystemFonts.Find(InFontName);
	if (!SystemFont || !IsValid(SystemFont->Font))
	{
		return false;
	}
	
	const FText3DEditorFont* ProjectFont = ProjectFonts.Find(InFontName);
	if (!ProjectFont || !IsValid(ProjectFont->Font))
	{
		return false;
	}

	return SystemFont->FontStyleFlags == ProjectFont->FontStyleFlags
		&& SystemFont->FontFaces.Num() == ProjectFont->FontFaces.Num();
}

void UText3DEditorFontSubsystem::OnSaveImportedFonts(UWorld* InWorld, FObjectPreSaveContext InContext)
{
	for (UPackage* Package : PackagesToSave)
	{
		const FString FontAssetFileName = FPackageName::LongPackageNameToFilename(Package->GetPathName(), FPackageName::GetAssetPackageExtension());
		if (!FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FontAssetFileName))
		{
			FSavePackageArgs FontSavePackageArgs;
			FontSavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
			UPackage::SavePackage(Package, nullptr, *FontAssetFileName, FontSavePackageArgs);
		}
	}

	PackagesToSave.Empty();
}

void UText3DEditorFontSubsystem::OnAssetLoaded()
{
	bInitialized = true;
	LoadProjectFonts();
	LoadSystemFonts();
}

void UText3DEditorFontSubsystem::OnAssetAdded(const FAssetData& InAssetData)
{
	if (!bInitialized)
	{
		return;
	}

	if (InAssetData.AssetClassPath != UFont::StaticClass()->GetClassPathName())
	{
		return;
	}

	if (UFont* Font = Cast<UFont>(InAssetData.GetAsset()))
	{
		RegisterProjectFont(Font);
	}
}

void UText3DEditorFontSubsystem::OnAssetDeleted(UObject* InObject)
{
	if (!bInitialized)
	{
		return;
	}

	if (!IsValid(InObject))
	{
		return;
	}

	if (const UFont* Font = Cast<UFont>(InObject))
	{
		UnregisterProjectFont(Font);
	}
}

void UText3DEditorFontSubsystem::LoadProjectFonts()
{
	if (!bInitialized)
	{
		return;
	}

	ProjectFonts.Empty();

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> AssetDataList;
	const UClass* Class = UFont::StaticClass();

	const FTopLevelAssetPath AssetPath(Class->GetPathName());
	AssetRegistryModule.Get().GetAssetsByClass(AssetPath, AssetDataList);

	for (const FAssetData& AssetData : AssetDataList)
	{
		UFont* Font = Cast<UFont>(AssetData.GetAsset());

		if (IsValid(Font) && Font->GetPackage())
		{
			Font->GetPackage()->FullyLoad();
			RegisterProjectFont(Font);
		}
	}
}

void UText3DEditorFontSubsystem::LoadSystemFonts()
{
	if (!bInitialized || !FSlateApplication::IsInitialized())
	{
		return;
	}

	SystemFonts.Empty();

	TMap<FString, FText3DFontFamily> FontFamilies;
	UE::Text3D::Private::Fonts::GetSystemFontInfo(FontFamilies);

	if (FontFamilies.IsEmpty())
	{
		return;
	}

	for (const TPair<FString, FText3DFontFamily>& FontFamilyPair : FontFamilies)
	{
		RegisterSystemFont(FontFamilyPair.Value);
	}

	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->SystemFontNames = GetSystemFontNames();
	}
}

bool UText3DEditorFontSubsystem::UnregisterProjectFont(const UFont* InFont)
{
	if (!IsValid(InFont))
	{
		return false;
	}

	FString FontName;
	if (!UE::Text3D::Utilities::Font::GetSanitizeFontName(InFont, FontName))
	{
		return false;
	}

	FText3DEditorFont* EditorFont = ProjectFonts.Find(FontName);
	if (!EditorFont)
	{
		return false;
	}

	OnProjectFontUnregisteredDelegate.Broadcast(FontName);

	ProjectFonts.Remove(FontName);

	return true;
}

bool UText3DEditorFontSubsystem::RegisterSystemFont(const FText3DFontFamily& InFontFamily)
{
	if (InFontFamily.FontFacePaths.IsEmpty())
	{
		return false;
	}

	FString SanitizedFontFamilyName = InFontFamily.FontFamilyName;
	UE::Text3D::Utilities::Font::SanitizeFontName(SanitizedFontFamilyName);

	UFont* NewFont = NewObject<UFont>(this, *SanitizedFontFamilyName, RF_Public | RF_Standalone);
	NewFont->ImportOptions.FontName = SanitizedFontFamilyName;
	NewFont->LegacyFontName = FName(SanitizedFontFamilyName);
	NewFont->FontCacheType = EFontCacheType::Runtime;

	UFontFileImportFactory* FontFaceFactory = NewObject<UFontFileImportFactory>();
	FontFaceFactory->SetAutomatedAssetImportData(NewObject<UAutomatedAssetImportData>());

	for (const TPair<FString, FString>& FontFacePath : InFontFamily.FontFacePaths)
	{
		if (!IsFontFileSupported(FontFacePath.Value))
		{
			continue;
		}

		FString SanitizedFontFaceName = FontFacePath.Key;
		UE::Text3D::Utilities::Font::SanitizeFontName(SanitizedFontFaceName);
		const FString FontFaceAssetName = SanitizedFontFamilyName + "_" + SanitizedFontFaceName;

		bool bCanceled = false;
		if (UFontFace* NewFontFace = Cast<UFontFace>(FontFaceFactory->ImportObject(UFontFace::StaticClass(), this, *FontFaceAssetName, RF_Public | RF_Standalone, FontFacePath.Value, TEXT(""), bCanceled)))
		{
			FTypefaceEntry& Entry = NewFont->GetMutableInternalCompositeFont().DefaultTypeface.Fonts.AddDefaulted_GetRef();
			Entry.Name = FName(FName::NameToDisplayString(SanitizedFontFaceName, false));
			Entry.Font = FFontData(NewFontFace);
		}
	}

	FontFaceFactory->MarkAsGarbage();

	// Fallback
	if (const UText3DProjectSettings* Text3DProjectSettings = UText3DProjectSettings::Get())
	{
		FTypefaceEntry& FallbackTypefaceEntry = NewFont->GetMutableInternalCompositeFont().FallbackTypeface.Typeface.Fonts.AddDefaulted_GetRef();
		FallbackTypefaceEntry.Font = FFontData(Text3DProjectSettings->GetFallbackFontFace());
		FallbackTypefaceEntry.Name = FName("Regular");
	}
	
	TArray<UFontFace*> FontFaces;
	UE::Text3D::Utilities::Font::GetFontFaces(NewFont, FontFaces);

	EText3DFontStyleFlags FontStyleFlags;
	UE::Text3D::Utilities::Font::GetFontStyle(NewFont, FontStyleFlags);

	FText3DEditorFont& RegisteredFont = SystemFonts.FindOrAdd(SanitizedFontFamilyName);
	
	const bool bHasChanged = RegisteredFont.FontName != SanitizedFontFamilyName
		|| RegisteredFont.Font != NewFont
		|| RegisteredFont.FontLocationFlags != EText3DEditorFontLocationFlags::System
		|| RegisteredFont.FontStyleFlags != FontStyleFlags
		|| RegisteredFont.FontFaces.Num() != FontFaces.Num();

	RegisteredFont.FontName = SanitizedFontFamilyName;
	RegisteredFont.FontLocationFlags = EText3DEditorFontLocationFlags::System;
	RegisteredFont.FontStyleFlags = FontStyleFlags;
	RegisteredFont.Font = NewFont;
	RegisteredFont.FontFaces = FontFaces;

	if (bHasChanged)
	{
		SystemFonts.KeyStableSort([](const FString& InFontA, const FString& InFontB)
		{
			return InFontA < InFontB;
		});

		OnSystemFontRegisteredDelegate.Broadcast(SanitizedFontFamilyName);
	}

	return true;
}

bool UText3DEditorFontSubsystem::UnregisterSystemFont(const FString& InFontName)
{
	FText3DEditorFont* EditorFont = ProjectFonts.Find(InFontName);
	if (!EditorFont)
	{
		return false;
	}

	OnSystemFontUnregisteredDelegate.Broadcast(InFontName);

	SystemFonts.Remove(InFontName);

	return true;
}

UFont* UText3DEditorFontSubsystem::ResolveFontByName(const FString& InFontName)
{
	if (const FText3DEditorFont* EditorFont = GetProjectFont(InFontName))
	{
		return EditorFont->Font;
	}

	return nullptr;
}

bool UText3DEditorFontSubsystem::RegisterProjectFont(UFont* InFont)
{
	if (!IsValid(InFont))
	{
		return false;
	}

	if (InFont->ImportOptions.bUseDistanceFieldAlpha)
	{
		UE_LOGF(LogText3DEditor, Log, "Ignoring distance field font %ls", *InFont->GetName());
		return false;
	}

	if (!InFont->IsAsset())
	{
		UE_LOGF(LogText3DEditor, Log, "Cannot register font that is not an asset %ls", *InFont->GetName());
		return false;
	}

	if (!InFont->GetCompositeFont() || InFont->GetCompositeFont()->DefaultTypeface.Fonts.IsEmpty() || InFont->RuntimeFontSource != ERuntimeFontSource::Asset)
	{
		UE_LOGF(LogText3DEditor, Log, "No composite font found for font %ls", *InFont->GetName());
		return false;
	}

	FString SanitizedFontName;
	if (!UE::Text3D::Utilities::Font::GetSanitizeFontName(InFont, SanitizedFontName))
	{
		UE_LOGF(LogText3DEditor, Warning, "Could not retrieve sanitized font name %ls", *InFont->GetName());
		return false;
	}

	EText3DFontStyleFlags FontStyleFlags;
	if (!UE::Text3D::Utilities::Font::GetFontStyle(InFont, FontStyleFlags))
	{
		UE_LOGF(LogText3DEditor, Warning, "Could not retrieve font style for %ls", *InFont->GetName());
		return false;
	}

	TArray<UFontFace*> FontFaces;
	if (!UE::Text3D::Utilities::Font::GetFontFaces(InFont, FontFaces))
	{
		UE_LOGF(LogText3DEditor, Warning, "Could not retrieve font faces for %ls", *InFont->GetName());
		return false;
	}

	FText3DEditorFont& RegisteredFont = ProjectFonts.FindOrAdd(SanitizedFontName);

	const bool bHasChanged = RegisteredFont.FontName != SanitizedFontName
		|| RegisteredFont.Font != InFont
		|| RegisteredFont.FontLocationFlags != EText3DEditorFontLocationFlags::Project
		|| RegisteredFont.FontStyleFlags != FontStyleFlags
		|| RegisteredFont.FontFaces.Num() != FontFaces.Num();
	
	RegisteredFont.FontName = SanitizedFontName;
	RegisteredFont.Font = InFont;
	RegisteredFont.FontLocationFlags = EText3DEditorFontLocationFlags::Project;
	RegisteredFont.FontStyleFlags = FontStyleFlags;
	RegisteredFont.FontFaces = FontFaces;

	if (bHasChanged)
	{
		ProjectFonts.KeyStableSort([](const FString& InFontA, const FString& InFontB)
		{
			return InFontA < InFontB;
		});

		OnProjectFontRegisteredDelegate.Broadcast(SanitizedFontName);
	}

	return true;
}
