// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Text3DTypes.h"
#include "UObject/ObjectPtr.h"
#include "Text3DEditorFontSubsystem.generated.h"

class FObjectPreSaveContext;
class UFontFace;
class UFont;
class UWorld;

UENUM()
enum class EText3DEditorFontLocationFlags : uint8
{
	None = 0,
	/** Font is available in the project */
	Project = 1 << 0,
	/** Font is available on the system */
	System = 1 << 2
};

/** Helper to manipulate a font in editor */
USTRUCT()
struct FText3DEditorFont
{
	GENERATED_BODY()

	/** Current location of the font */
	UPROPERTY()
	EText3DEditorFontLocationFlags FontLocationFlags = EText3DEditorFontLocationFlags::None;

	/** The font name to lookup the font */
	UPROPERTY()
	FString FontName;

	/** The actual font object if available */
	UPROPERTY()
	TObjectPtr<UFont> Font;

	/** Font faces available composing this font */
	UPROPERTY()
	TArray<TObjectPtr<UFontFace>> FontFaces;

	/** Font style flags */
	UPROPERTY()
	EText3DFontStyleFlags FontStyleFlags = EText3DFontStyleFlags::None;
};

UCLASS()
class UText3DEditorFontSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnProjectFontRegistered, const FString& InFontName)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnProjectFontUnregistered, const FString& InFontName)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSystemFontRegistered, const FString& InFontName)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSystemFontUnregistered, const FString& InFontName)
	
	static UText3DEditorFontSubsystem* Get();

	static bool IsFontFileSupported(const FString& InFontFilePath);

	FOnProjectFontRegistered::RegistrationType& OnProjectFontRegistered()
	{
		return OnProjectFontRegisteredDelegate;
	}

	FOnProjectFontUnregistered::RegistrationType& OnProjectFontUnregistered()
	{
		return OnProjectFontUnregisteredDelegate;
	}

	FOnSystemFontRegistered::RegistrationType& OnSystemFontRegistered()
	{
		return OnSystemFontRegisteredDelegate;
	}

	FOnSystemFontUnregistered::RegistrationType& OnSystemFontUnregistered()
	{
		return OnSystemFontUnregisteredDelegate;
	}

	/** Imports system font into project font */
	bool ImportSystemFont(const FString& InFontName);

	TArray<FString> GetProjectFontNames() const;
	TArray<FString> GetSystemFontNames() const;
	TArray<FString> GetFavoriteFontNames() const;

	const FText3DEditorFont* GetEditorFont(const FString& InFontName) const;
	const FText3DEditorFont* GetSystemFont(const FString& InFontName) const;
	const FText3DEditorFont* GetProjectFont(const FString& InFontName) const;
	const FText3DEditorFont* FindEditorFont(const UFont* InFont) const;

protected:
	//~ Begin UEditorSubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	//~ End UEditorSubsystem

	bool IsProjectFontUpToDate(const FString& InFontName);
	void OnSaveImportedFonts(UWorld* InWorld, FObjectPreSaveContext InContext);

	void OnAssetLoaded();
	void OnAssetAdded(const FAssetData& InAssetData);
	void OnAssetDeleted(UObject* InObject);

	void LoadProjectFonts();
	bool RegisterProjectFont(UFont* InFont);
	bool UnregisterProjectFont(const UFont* InFont);

	void LoadSystemFonts();
	bool RegisterSystemFont(const FText3DFontFamily& InFontFamily);
	bool UnregisterSystemFont(const FString& InFontName);
	
	UFont* ResolveFontByName(const FString& InFontName);

	/** Registered font available to use within the project */
	UPROPERTY()
	TMap<FString, FText3DEditorFont> ProjectFonts;
	
	/** Registered font available to use within the system */
	UPROPERTY()
	TMap<FString, FText3DEditorFont> SystemFonts;

	UPROPERTY()
	TArray<TObjectPtr<UPackage>> PackagesToSave;

	/** Is the subsystem initialized */
	bool bInitialized = false;
	
	FOnProjectFontRegistered OnProjectFontRegisteredDelegate;
	FOnProjectFontUnregistered OnProjectFontUnregisteredDelegate;
	FOnSystemFontRegistered OnSystemFontRegisteredDelegate;
	FOnSystemFontUnregistered OnSystemFontUnregisteredDelegate;
};
