// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text3DEditorModule.h"

#include "Commands/Text3DEditorFontSelectorCommands.h"
#include "Customizations/Text3DEditorCharacterPropertyTypeCustomization.h"
#include "Customizations/Text3DEditorFontPropertyTypeCustomization.h"
#include "Customizations/Text3DEditorHorizontalPropertyTypeCustomization.h"
#include "Customizations/Text3DEditorVerticalPropertyTypeCustomization.h"
#include "Customizations/Text3DEditorTextComponentDetailCustomization.h"
#include "Engine/Font.h"
#include "Extensions/Text3DDefaultCharacterExtension.h"
#include "Logs/Text3DEditorLogs.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styles/Text3DEditorStyle.h"
#include "Text3DComponent.h"
#include "Text3DTypes.h"
#include "UObject/ReflectedTypeAccessors.h"

DEFINE_LOG_CATEGORY(LogText3DEditor);

IMPLEMENT_MODULE(FText3DEditorModule, Text3DEditor)

void FText3DEditorModule::StartupModule()
{
	using namespace UE::Text3DEditor::Customization;

	// Init style set
	FText3DEditorStyle::Get();

	FText3DEditorFontSelectorCommands::Register();
	
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredTypeNames.Add_GetRef(StaticEnum<EText3DHorizontalTextAlignment>()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FText3DEditorHorizontalPropertyTypeCustomization::MakeInstance)
		, MakeShared<FText3DEditorHorizontalPropertyTypeIdentifier>());

	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredTypeNames.Add_GetRef(StaticEnum<EText3DVerticalTextAlignment>()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FText3DEditorVerticalPropertyTypeCustomization::MakeInstance)
		, MakeShared<FText3DEditorVerticalPropertyTypeIdentifier>());
	
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredTypeNames.Add_GetRef(UFont::StaticClass()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FText3DEditorFontPropertyTypeCustomization::MakeInstance)
		, MakeShared<FText3DEditorFontPropertyTypeIdentifier>());

	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredTypeNames.Add_GetRef(FUInt16Property::StaticClass()->GetFName())
	 	, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FText3DEditorCharacterPropertyTypeCustomization::MakeInstance)
	 	, MakeShared<FText3DEditorCharacterPropertyTypeIdentifier>());

	PropertyModule.RegisterCustomClassLayout(RegisteredTypeNames.Add_GetRef(UText3DComponent::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FText3DEditorTextComponentDetailCustomization::MakeInstance));
}

void FText3DEditorModule::ShutdownModule()
{
	FText3DEditorFontSelectorCommands::Unregister();

	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(TEXT("PropertyEditor")))
	{
		for (const FName& RegisteredTypeName : RegisteredTypeNames)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(RegisteredTypeName);
			PropertyModule->UnregisterCustomClassLayout(RegisteredTypeName);
		}
	}

	RegisteredTypeNames.Empty();
}
