// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FText3DEditorModule final : public IModuleInterface
{
public:
	FText3DEditorModule() = default;

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	TArray<FName> RegisteredTypeNames;
};