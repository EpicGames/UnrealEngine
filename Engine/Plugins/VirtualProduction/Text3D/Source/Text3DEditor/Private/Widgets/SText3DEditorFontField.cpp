// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SText3DEditorFontField.h"

#include "Editor.h"
#include "Engine/Font.h"
#include "SlateOptMacros.h"
#include "Settings/Text3DProjectSettings.h"
#include "Styling/StyleColors.h"
#include "Subsystems/Text3DEditorFontSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::AvaTextEditor::Private
{
	static constexpr float DefaultFontSize = 12.0f;
}

#define LOCTEXT_NAMESPACE "Text3DEditorFontField"

void SText3DEditorFontField::Construct(const FArguments& InArgs)
{
	FontItemWeak = InArgs._FontItem;

	const FVector2D Icon16x16(16.0f, 16.0f);

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(400.f)
		.MaxDesiredWidth(400.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Clipping(EWidgetClipping::ClipToBoundsAlways) // we don't want list elements to render outside the box
		.ToolTipText(this, &SText3DEditorFontField::GetFontTooltipText)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SScaleBox)
				.Visibility(InArgs._ShowFavoriteButton ? EVisibility::Visible : EVisibility::Collapsed)
				.Stretch(EStretch::ScaleToFit)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("ToggleFavorite", "Mark Font as Favorite"))
					.OnClicked(this, &SText3DEditorFontField::OnToggleFavoriteClicked)
					.Visibility(this, &SText3DEditorFontField::GetFavoriteVisibility)
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SText3DEditorFontField::GetToggleFavoriteColor)
						.Image(FAppStyle::GetBrush("Icons.Star"))
						.DesiredSizeOverride(Icon16x16)
					]
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(LeftFontNameText, STextBlock)
				.ColorAndOpacity(FSlateColor(EStyleColor::White))
				.Justification(ETextJustify::Left)
				.OverflowPolicy(ETextOverflowPolicy::Clip)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SAssignNew(RightFontNameText, STextBlock)
				.ColorAndOpacity(FSlateColor(EStyleColor::White))
				.Justification(ETextJustify::Right)
				.OverflowPolicy(ETextOverflowPolicy::Clip)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SText3DEditorFontField::OnBrowseToAssetClicked)
				.ToolTipText(LOCTEXT( "BrowseButtonToolTipText", "Browse to Font asset in Content Browser"))
				.Visibility(this, &SText3DEditorFontField::GetLocallyAvailableIconVisibility)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.BrowseContent"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	];

	UpdateFont();
}

void SText3DEditorFontField::UpdateFont()
{
	const TSharedPtr<FString> FontItem = FontItemWeak.Pin();

	if (!FontItem.IsValid())
	{
		return;
	}

	const UText3DEditorFontSubsystem* FontEditorSubsystem = UText3DEditorFontSubsystem::Get();
	const FText3DEditorFont* EditorFont = FontEditorSubsystem->GetEditorFont(*FontItem.Get());
	
	if (EditorFont && EditorFont->Font)
	{
		LeftFontNameText->SetText(FText::FromString(EditorFont->FontName));

		RightFontNameText->SetText(FText::FromName(EditorFont->Font->LegacyFontName));
		RightFontNameText->SetVisibility(EditorFont->Font->LegacyFontName.IsNone() ? EVisibility::Collapsed : EVisibility::Visible);

		if (EditorFont->Font->GetCompositeFont())
		{
			FSlateFontInfo SlateFont(EditorFont->Font, UE::AvaTextEditor::Private::DefaultFontSize);
			LeftFontNameText->SetFont(SlateFont);
		}
	}
}

ECheckBoxState SText3DEditorFontField::GetFavoriteState() const
{
	if (const TSharedPtr<FString> FontItem = FontItemWeak.Pin())
	{
		if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
		{
			return Text3DSettings->GetFavoriteFonts().Contains(*FontItem.Get()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Unchecked;
}

FSlateColor SText3DEditorFontField::GetToggleFavoriteColor() const
{
	return GetFavoriteState() == ECheckBoxState::Checked ? FSlateColor(EStyleColor::AccentBlue) : FSlateColor(EStyleColor::Foreground); 
}

FReply SText3DEditorFontField::OnToggleFavoriteClicked()
{
	const TSharedPtr<FString> FontItem = FontItemWeak.Pin();

	if (!FontItem.IsValid())
	{
		return FReply::Handled();
	}

	UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable();
	if (!Text3DSettings)
	{
		return FReply::Handled();
	}

	if (GetFavoriteState() == ECheckBoxState::Checked)
	{
		Text3DSettings->RemoveFavoriteFont(*FontItem.Get());
	}
	else
	{
		Text3DSettings->AddFavoriteFont(*FontItem.Get());
	}

	return FReply::Handled();
}

EVisibility SText3DEditorFontField::GetFavoriteVisibility() const
{
	return IsHovered()
		|| GetFavoriteState() == ECheckBoxState::Checked
		? EVisibility::Visible
		: EVisibility::Hidden;
}

FReply SText3DEditorFontField::OnBrowseToAssetClicked() const
{
	const TSharedPtr<FString> FontItem = FontItemWeak.Pin();

	if (!FontItem.IsValid())
	{
		return FReply::Handled();
	}

	UText3DEditorFontSubsystem* FontEditorSubsystem = UText3DEditorFontSubsystem::Get();
	const FText3DEditorFont* EditorFont = FontEditorSubsystem->GetProjectFont(*FontItem.Get());
	
	if (EditorFont && EditorFont->Font)
	{
		const TArray<UObject*> ObjectsToFocus {EditorFont->Font};
		GEditor->SyncBrowserToObjects(ObjectsToFocus);
	}

	return FReply::Handled();
}

EVisibility SText3DEditorFontField::GetLocallyAvailableIconVisibility() const
{
	const TSharedPtr<FString> FontItem = FontItemWeak.Pin();

	if (!FontItem.IsValid())
	{
		return EVisibility::Collapsed;
	}

	UText3DEditorFontSubsystem* FontEditorSubsystem = UText3DEditorFontSubsystem::Get();
	const FText3DEditorFont* EditorFont = FontEditorSubsystem->GetProjectFont(*FontItem.Get());
	return EditorFont && EditorFont->Font ? EVisibility::Visible : EVisibility::Hidden;
}

FText SText3DEditorFontField::GetFontTooltipText() const
{
	if (!GetLocallyAvailableIconVisibility().IsVisible())
	{
		return LOCTEXT("SelectToImportSystemFont", "Select and import system font into project");
	}

	return LOCTEXT("ProjectFontAvailable", "Select to use project font");
}

#undef LOCTEXT_NAMESPACE
