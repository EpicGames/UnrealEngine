// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
class FText;
class STextBlock;
enum class ECheckBoxState : uint8;
struct EVisibility;
struct FSlateColor;

class SText3DEditorFontField : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SText3DEditorFontField) {}
		SLATE_ARGUMENT(TSharedPtr<FString>, FontItem)
		SLATE_ARGUMENT(bool, ShowFavoriteButton)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	void UpdateFont();

	FReply OnToggleFavoriteClicked();
	ECheckBoxState GetFavoriteState() const;
	FSlateColor GetToggleFavoriteColor() const;
	EVisibility GetFavoriteVisibility() const;

	FReply OnBrowseToAssetClicked() const;
	EVisibility GetLocallyAvailableIconVisibility() const;

	FText GetFontTooltipText() const;

	TSharedPtr<STextBlock> LeftFontNameText;
	TSharedPtr<STextBlock> RightFontNameText;
	TWeakPtr<FString> FontItemWeak;
};
