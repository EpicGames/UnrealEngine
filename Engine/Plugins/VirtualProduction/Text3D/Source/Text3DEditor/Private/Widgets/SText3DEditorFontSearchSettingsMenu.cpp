// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SText3DEditorFontSearchSettingsMenu.h"

#include "Commands/Text3DEditorFontSelectorCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Settings/Text3DProjectSettings.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "Widgets/Images/SImage.h"

void SText3DEditorFontSearchSettingsMenu::Construct(const FArguments& InArgs)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	check(ToolMenus)

	BindCommands();

	static const FName FontSearchSettingsToolbar = TEXT("FontSearchSettingsToolbar");
	if (!ToolMenus->IsMenuRegistered(FontSearchSettingsToolbar))
	{
		UToolMenu* ToolBar = ToolMenus->RegisterMenu(FontSearchSettingsToolbar, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolBar->SetStyleSet(&FAppStyle::Get());
		ToolBar->StyleName = "ViewportLayoutToolbar";

		const FText3DEditorFontSelectorCommands& FontSelectorCommands = FText3DEditorFontSelectorCommands::Get();
		ToolBar->AddMenuEntry(TEXT("ShowMonospacedFonts"), FToolMenuEntry::InitToolBarButton(FontSelectorCommands.ShowMonospacedFonts));
		ToolBar->AddMenuEntry(TEXT("ShowBoldFonts"), FToolMenuEntry::InitToolBarButton(FontSelectorCommands.ShowBoldFonts));
		ToolBar->AddMenuEntry(TEXT("ShowItalicFonts"), FToolMenuEntry::InitToolBarButton(FontSelectorCommands.ShowItalicFonts));
	}

	ChildSlot
	[
		ToolMenus->GenerateWidget(FontSearchSettingsToolbar, FToolMenuContext(CommandList))
	];
}

void SText3DEditorFontSearchSettingsMenu::BindCommands()
{
	const FText3DEditorFontSelectorCommands& FontSelectorCommands = FText3DEditorFontSelectorCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	// Monospaced filter
	CommandList->MapAction(FontSelectorCommands.ShowMonospacedFonts
		, FExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowMonospacedToggle_Execute)
		, FCanExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowMonospacedToggle_CanExecute)
		, FIsActionChecked::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowMonospacedToggle_IsChecked));

	// Bold filter
	CommandList->MapAction(FontSelectorCommands.ShowBoldFonts
		, FExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowBoldToggle_Execute)
		, FCanExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowBoldToggle_CanExecute)
		, FIsActionChecked::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowBoldToggle_IsChecked));

	// Italic filter
	CommandList->MapAction(FontSelectorCommands.ShowItalicFonts
		, FExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowItalicToggle_Execute)
		, FCanExecuteAction::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowItalicToggle_CanExecute)
		, FIsActionChecked::CreateSP(this, &SText3DEditorFontSearchSettingsMenu::ShowItalicToggle_IsChecked));
}

bool SText3DEditorFontSearchSettingsMenu::ShowMonospacedToggle_IsChecked()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		return Text3DSettings->GetShowOnlyMonospaced();
	}

	return false;
}

bool SText3DEditorFontSearchSettingsMenu::ShowBoldToggle_IsChecked()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		return Text3DSettings->GetShowOnlyBold();
	}

	return false;
}

bool SText3DEditorFontSearchSettingsMenu::ShowItalicToggle_IsChecked()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		return Text3DSettings->GetShowOnlyItalic();
	}

	return false;
}

void SText3DEditorFontSearchSettingsMenu::ShowMonospacedToggle_Execute()
{
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->SetShowOnlyMonospaced(!Text3DSettings->GetShowOnlyMonospaced());
	}
}

void SText3DEditorFontSearchSettingsMenu::ShowBoldToggle_Execute()
{
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->SetShowOnlyBold(!Text3DSettings->GetShowOnlyBold());
	}
}

void SText3DEditorFontSearchSettingsMenu::ShowItalicToggle_Execute()
{
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->SetShowOnlyItalic(!Text3DSettings->GetShowOnlyItalic());
	}
}
