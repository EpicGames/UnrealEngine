// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SText3DEditorFontSelector;

/** Widget to apply settings on font selector */
class SText3DEditorFontSearchSettingsMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SText3DEditorFontSearchSettingsMenu)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	void BindCommands();

	bool ShowMonospacedToggle_IsChecked();
	bool ShowBoldToggle_IsChecked();
	bool ShowItalicToggle_IsChecked();

	void ShowMonospacedToggle_Execute();
	void ShowBoldToggle_Execute();
	void ShowItalicToggle_Execute();

	bool ShowMonospacedToggle_CanExecute() const
	{
		return true;
	}

	bool ShowBoldToggle_CanExecute() const
	{
		return true;
	}

	bool ShowItalicToggle_CanExecute() const
	{
		return true;
	}

	TSharedPtr<FUICommandList> CommandList;
};
