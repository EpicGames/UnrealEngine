// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SText3DEditorFontSelector.h"

#include "Engine/Font.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Misc/EnumerateRange.h"
#include "PropertyHandle.h"
#include "Settings/Text3DProjectSettings.h"
#include "SText3DEditorFontField.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Subsystems/Text3DEditorFontSubsystem.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SText3DEditorFontSearchSettingsMenu.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "Text3DEditorFontSelector"

namespace UE::Private::Text3D
{
	class SFontSelectorRow : public STableRow<TSharedPtr<FString>>
	{
	public:
		SLATE_BEGIN_ARGS(SFontSelectorRow)
			: _Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
			, _Content()
			, _Padding(FMargin(0))
		{}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
		{
			STableRow<TSharedPtr<FString>>::Construct(
				STableRow<TSharedPtr<FString>>::FArguments()
				.Style(InArgs._Style)
				.Padding(InArgs._Padding)
				.Content()
				[
					InArgs._Content.Widget
				]
				, InOwnerTable
			);
		}
	};
}

SText3DEditorFontSelector::~SText3DEditorFontSelector()
{
	if (UObjectInitialized())
	{
		UnbindDelegates();
	}
}

void SText3DEditorFontSelector::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	FontPropertyHandle = InPropertyHandle;

	FontPropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &SText3DEditorFontSelector::OnPropertyResetToDefault));

	const TSharedPtr<SWidget> ComboBoxMenuContent = SNew(SBorder)
		.Padding(2.0f)
		.BorderBackgroundColor(FSlateColor(EStyleColor::AccentBlue))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(7.0f, 6.0f)
			.AutoHeight()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SText3DEditorFontSelector::OnFilterTextChanged)
					.OnKeyDownHandler(this, &SText3DEditorFontSelector::OnSearchFieldKeyDown)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SText3DEditorFontSearchSettingsMenu)
				]
			]

			// FAVORITE FONTS SECTION //
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Center)
			.Padding(2.0f)
			[
				SAssignNew(FavoriteSeparator, SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(FavoriteLabel, STextBlock)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredHeight(200.0f)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(FavoriteFontsListView, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&FavoriteFontsItems)
					.OnGenerateRow(this, &SText3DEditorFontSelector::GenerateMenuItemRow)
					.OnSelectionChanged(this, &SText3DEditorFontSelector::OnFavoriteFontItemSelectionChanged)
					.SelectionMode(ESelectionMode::Single)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				]
			]

			// PROJECT FONTS SECTION //
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Center)
			.Padding(2.0f)
			[
				SAssignNew(ProjectSeparator, SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(ProjectLabel, STextBlock)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredHeight(200.0f)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ProjectFontsListView, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&ProjectFontsItems)
					.OnGenerateRow(this, &SText3DEditorFontSelector::GenerateMenuItemRow)
					.OnSelectionChanged(this, &SText3DEditorFontSelector::OnProjectFontItemSelectionChanged)
					.SelectionMode(ESelectionMode::Single)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				]
			]

			// SYSTEM FONTS SECTION //
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Center)
			.Padding(2.0f)
			[
				SAssignNew(SystemSeparator, SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(SystemLabel, STextBlock)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredHeight(200.0f)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(SystemFontsListView, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&SystemFontItems)
					.OnGenerateRow(this, &SText3DEditorFontSelector::GenerateMenuItemRow)
					.OnSelectionChanged(this, &SText3DEditorFontSelector::OnSystemFontItemSelectionChanged)
					.SelectionMode(ESelectionMode::Single)
					.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				]
			]
		];

	ChildSlot
	.Padding(0.f)
	[
		SAssignNew(ComboButton, SComboButton)
		.CollapseMenuOnParentFocus(true)
		.Method(EPopupMethod::UseCurrentWindow)
		.MenuPlacement(EMenuPlacement::MenuPlacement_MenuLeft)
		.ContentPadding(0.f)
		.ButtonContent()
		[
			SAssignNew(FontContainer, SBox)
			.Padding(0.f)
		]
		.MenuContent()
		[
			ComboBoxMenuContent.ToSharedRef()
		]
		.HasDownArrow(true)
		.IsFocusable(true)
		.ForegroundColor(FSlateColor(EStyleColor::Dropdown))
		.OnMenuOpenChanged(this, &SText3DEditorFontSelector::OnMenuOpenChanged)
	];

	UnbindDelegates();
	BindDelegates();

	UpdateItems();
}

void SText3DEditorFontSelector::UpdateSeparatorsVisibility() const
{
	if (SystemFontItems.IsEmpty())
	{
		SystemSeparator->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		SystemSeparator->SetVisibility(EVisibility::Visible);
		SystemLabel->SetText(GetSystemFontLabel());
	}

	if (FavoriteFontsItems.IsEmpty())
	{
		FavoriteSeparator->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		FavoriteSeparator->SetVisibility(EVisibility::Visible);
		FavoriteLabel->SetText(GetFavoriteFontLabel());
	}

	if (ProjectFontsItems.IsEmpty())
	{
		ProjectSeparator->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		ProjectSeparator->SetVisibility(EVisibility::Visible);
		ProjectLabel->SetText(GetProjectFontLabel());
	}
}

void SText3DEditorFontSelector::BindDelegates()
{
	if (UText3DEditorFontSubsystem* FontSubsystem = UText3DEditorFontSubsystem::Get())
	{
		FontSubsystem->OnProjectFontRegistered().AddSP(this, &SText3DEditorFontSelector::OnProjectFontRegistered);
		FontSubsystem->OnProjectFontUnregistered().AddSP(this, &SText3DEditorFontSelector::OnProjectFontUnregistered);
		FontSubsystem->OnSystemFontRegistered().AddSP(this, &SText3DEditorFontSelector::OnSystemFontRegistered);
		FontSubsystem->OnSystemFontUnregistered().AddSP(this, &SText3DEditorFontSelector::OnSystemFontUnregistered);
	}

	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->OnSettingChanged().AddSP(this, &SText3DEditorFontSelector::OnSettingsChanged);
	}
}

void SText3DEditorFontSelector::UnbindDelegates() const
{
	if (UText3DEditorFontSubsystem* FontSubsystem = UText3DEditorFontSubsystem::Get())
	{
		FontSubsystem->OnProjectFontRegistered().RemoveAll(this);
		FontSubsystem->OnProjectFontUnregistered().RemoveAll(this);
		FontSubsystem->OnSystemFontRegistered().RemoveAll(this);
		FontSubsystem->OnSystemFontUnregistered().RemoveAll(this);
	}
	
	if (UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::GetMutable())
	{
		Text3DSettings->OnSettingChanged().RemoveAll(this);
	}
}

void SText3DEditorFontSelector::UpdateItems()
{
	const UText3DEditorFontSubsystem* Text3DFontSubsystem = UText3DEditorFontSubsystem::Get();

	TArray<FString> FavoriteFontNames = Text3DFontSubsystem->GetFavoriteFontNames();
	TArray<FString> SystemFontNames = Text3DFontSubsystem->GetSystemFontNames();
	TArray<FString> ProjectFontNames = Text3DFontSubsystem->GetProjectFontNames();

	// Filter items
	ApplyItemFilters(FavoriteFontNames);
	ApplyItemFilters(SystemFontNames);
	ApplyItemFilters(ProjectFontNames);

	// Remove items
	for (TArray<TSharedPtr<FString>>::TIterator It(FavoriteFontsItems); It; ++It)
	{
		if (!It->IsValid())
		{
			It.RemoveCurrent();
		}

		const FString& Item = *It->Get();
		if (!FavoriteFontNames.Contains(Item))
		{
			It.RemoveCurrent();
		}
		else
		{
			FavoriteFontNames.Remove(Item);
		}
	}

	for (TArray<TSharedPtr<FString>>::TIterator It(ProjectFontsItems); It; ++It)
	{
		if (!It->IsValid())
		{
			It.RemoveCurrent();
		}

		const FString& Item = *It->Get();
		if (!ProjectFontNames.Contains(Item))
		{
			It.RemoveCurrent();
		}
		else
		{
			ProjectFontNames.Remove(Item);
		}
	}

	for (TArray<TSharedPtr<FString>>::TIterator It(SystemFontItems); It; ++It)
	{
		if (!It->IsValid())
		{
			It.RemoveCurrent();
		}

		const FString& Item = *It->Get();
		if (!SystemFontNames.Contains(Item))
		{
			It.RemoveCurrent();
		}
		else
		{
			SystemFontNames.Remove(Item);
		}
	}

	// Add items
	for (const FString& FavoriteFontName : FavoriteFontNames)
	{
		FavoriteFontsItems.Add(MakeShared<FString>(FavoriteFontName));
	}

	for (const FString& ProjectFontName : ProjectFontNames)
	{
		ProjectFontsItems.Add(MakeShared<FString>(ProjectFontName));
	}

	for (const FString& SystemFontName : SystemFontNames)
	{
		SystemFontItems.Add(MakeShared<FString>(SystemFontName));
	}
	
	UpdateSelectedItem();
	UpdateSeparatorsVisibility();

	FavoriteFontsListView->ClearSelection();
	ProjectFontsListView->ClearSelection();
	SystemFontsListView->ClearSelection();

	FavoriteFontsListView->RequestListRefresh();
	ProjectFontsListView->RequestListRefresh();
	SystemFontsListView->RequestListRefresh();
}

void SText3DEditorFontSelector::ApplyItemFilters(TArray<FString>& OutFilteredFontItems) const
{
	const FString SearchText = SearchBox->GetText().ToString();
	UText3DEditorFontSubsystem* Text3DFontSubsystem = UText3DEditorFontSubsystem::Get();
	const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get();

	for (TArray<FString>::TIterator It(OutFilteredFontItems); It; ++It)
	{
		const FString& Item = *It;
		if (!SearchText.IsEmpty() && !Item.Contains(SearchText))
		{
			It.RemoveCurrent();
			continue;
		}

		const FText3DEditorFont* EditorFont = Text3DFontSubsystem->GetEditorFont(Item);
		if (!EditorFont)
		{
			It.RemoveCurrent();
			continue;
		}

		if (Text3DSettings->GetShowOnlyMonospaced() && !EnumHasAnyFlags(EditorFont->FontStyleFlags, EText3DFontStyleFlags::Monospace))
		{
			It.RemoveCurrent();
			continue;
		}

		if (Text3DSettings->GetShowOnlyBold() && !EnumHasAnyFlags(EditorFont->FontStyleFlags, EText3DFontStyleFlags::Bold))
		{
			It.RemoveCurrent();
			continue;
		}

		if (Text3DSettings->GetShowOnlyItalic() && !EnumHasAnyFlags(EditorFont->FontStyleFlags, EText3DFontStyleFlags::Italic))
		{
			It.RemoveCurrent();
			continue;
		}
	}
}

void SText3DEditorFontSelector::OnProjectFontRegistered(const FString& InFontName)
{
	UpdateItems();
}

void SText3DEditorFontSelector::OnProjectFontUnregistered(const FString& InFontName)
{
	UpdateItems();
}

void SText3DEditorFontSelector::OnSystemFontRegistered(const FString& String)
{
	UpdateItems();
}

void SText3DEditorFontSelector::OnSystemFontUnregistered(const FString& String)
{
	UpdateItems();
}

void SText3DEditorFontSelector::OnSettingsChanged(UObject*, FPropertyChangedEvent&)
{
	UpdateItems();
}

void SText3DEditorFontSelector::OnFavoriteFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
{
	OnFontItemSelectionChanged(InItem, InSelectInfo);
}

void SText3DEditorFontSelector::OnProjectFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
{
	OnFontItemSelectionChanged(InItem, InSelectInfo);
}

void SText3DEditorFontSelector::OnSystemFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
{
	OnFontItemSelectionChanged(InItem, InSelectInfo);
}

FReply SText3DEditorFontSelector::OnSearchFieldKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		FSlateApplication::Get().DismissAllMenus();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<ITableRow> SText3DEditorFontSelector::GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	return SNew(UE::Private::Text3D::SFontSelectorRow, InOwnerTable)
		.Padding(FMargin(5.f))
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		[
			SNew(SText3DEditorFontField)
			.FontItem(InItem)
			.ShowFavoriteButton(true)
		];
}

void SText3DEditorFontSelector::OnMenuOpenChanged(bool bInOpen)
{
	if (bInOpen)
	{
		SystemFontsListView->ClearSelection();
		ProjectFontsListView->ClearSelection();
		FavoriteFontsListView->ClearSelection();
	}
}

FText SText3DEditorFontSelector::GetFavoriteFontLabel() const
{
	return FText::FromString(TEXT("Favorite ") + FString::FromInt(FavoriteFontsItems.Num()));
}

FText SText3DEditorFontSelector::GetProjectFontLabel() const
{
	return FText::FromString(TEXT("Project ") + FString::FromInt(ProjectFontsItems.Num()));
}

FText SText3DEditorFontSelector::GetSystemFontLabel() const
{
	return FText::FromString(TEXT("System ") + FString::FromInt(SystemFontItems.Num()));
}

void SText3DEditorFontSelector::OnFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectInfo)
{
	if (!InItem || SelectInfo == ESelectInfo::Type::OnNavigation)
	{
		return;
	}

	bool bNeedImport = false;
	if (ProjectFontsItems.Contains(InItem))
	{
		bNeedImport = false;
	}
	else if(SystemFontItems.Contains(InItem))
	{
		bNeedImport = true;
	}

	UText3DEditorFontSubsystem* FontSubsystem = UText3DEditorFontSubsystem::Get();
	const FText3DEditorFont* EditorFont = FontSubsystem->GetEditorFont(*InItem.Get());

	if (EditorFont && EditorFont->Font)
	{
		if (bNeedImport || EnumHasAnyFlags(EditorFont->FontLocationFlags, EText3DEditorFontLocationFlags::System))
		{
			FontSubsystem->ImportSystemFont(EditorFont->FontName);
		}

		FontPropertyHandle->SetValue(EditorFont->Font);
		UpdateSelectedItem();
	}

	ComboButton->SetIsOpen(false);
}

void SText3DEditorFontSelector::OnPropertyResetToDefault()
{
	if (const UText3DProjectSettings* Text3DSettings = UText3DProjectSettings::Get())
	{
		FontPropertyHandle->SetValue(Text3DSettings->GetFallbackFont());
		UpdateSelectedItem();
	}
}

void SText3DEditorFontSelector::UpdateSelectedItem()
{
	if (!FontPropertyHandle.IsValid())
	{
		return;
	}

	UFont* SelectedFont = nullptr;
	bool bMultiSelect = false;

	TArray<FString> FontObjectPaths;
	FontPropertyHandle->GetPerObjectValues(FontObjectPaths);

	for (TEnumerateRef<FString> FontObjectPath : EnumerateRange(FontObjectPaths))
	{
		UFont* Font = FindObject<UFont>(nullptr, **FontObjectPath);

		if (FontObjectPath.GetIndex() == 0)
		{
			SelectedFont = Font;
		}

		if (Font != SelectedFont)
		{
			SelectedFont = nullptr;
			bMultiSelect = true;
			break;
		}
	}

	UText3DEditorFontSubsystem* Text3DFontSubsystem = UText3DEditorFontSubsystem::Get();

	if (!SelectedFont)
	{
		if (bMultiSelect)
		{
			FontContainer->SetContent(
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor(EStyleColor::White))
				.Text(FText::FromString("Multiple Values"))
			);
		}
		else
		{
			FontContainer->SetContent(
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor(EStyleColor::White))
				.Text(FText::FromString("No Font Selected"))
			);
		}
	}
	else if (const FText3DEditorFont* Font = Text3DFontSubsystem->FindEditorFont(SelectedFont))
	{
		TSharedPtr<FString>* FoundItem = nullptr;

		if (EnumHasAnyFlags(Font->FontLocationFlags, EText3DEditorFontLocationFlags::Project))
		{
			FoundItem = ProjectFontsItems.FindByPredicate([Font](const TSharedPtr<FString>& InFontItem)
			{
				return InFontItem.IsValid() && (*InFontItem.Get()) == Font->FontName;
			});

			if (FoundItem)
			{
				ProjectFontsListView->RequestNavigateToItem(*FoundItem);
			}
		}
		else if (EnumHasAnyFlags(Font->FontLocationFlags, EText3DEditorFontLocationFlags::System))
		{
			FoundItem = SystemFontItems.FindByPredicate([Font](const TSharedPtr<FString>& InFontItem)
			{
				return InFontItem.IsValid() && (*InFontItem.Get()) == Font->FontName;
			});

			if (FoundItem)
			{
				SystemFontsListView->RequestNavigateToItem(*FoundItem);
			}
		}

		if (FoundItem)
		{
			FontContainer->SetContent(
				SNew(SText3DEditorFontField)
				.FontItem(*FoundItem)
				.ShowFavoriteButton(false)
			);
		}
	}
	else
	{
		FontContainer->SetContent(
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor(EStyleColor::White))
			.Text(FText::FromString("Unknown Font"))
		);
	}
}

void SText3DEditorFontSelector::OnFilterTextChanged(const FText& Text)
{
	UpdateItems();
}

#undef LOCTEXT_NAMESPACE
