// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"

class FReply;
class IPropertyHandle;
class SBox;
class SComboButton;
class SSearchBox;
class STableViewBase;
class STextBlock;
template <typename ItemType> class SListView;

class SText3DEditorFontSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SText3DEditorFontSelector)
		{}
	SLATE_END_ARGS()

	virtual ~SText3DEditorFontSelector() override;

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

protected:
	void BindDelegates();
	void UnbindDelegates() const;
	void UpdateItems();
	void ApplyItemFilters(TArray<FString>& OutFilteredFontItems) const;
	void UpdateSeparatorsVisibility() const;

	void OnProjectFontRegistered(const FString& InFontName);
	void OnProjectFontUnregistered(const FString& InFontName);
	void OnSystemFontRegistered(const FString& String);
	void OnSystemFontUnregistered(const FString& String);
	void OnSettingsChanged(UObject*, FPropertyChangedEvent&);

	void OnFavoriteFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);
	void OnProjectFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);
	void OnSystemFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);

	TEXT3DEDITOR_API void OnFontItemSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);
	void OnPropertyResetToDefault();
	void UpdateSelectedItem();

	FReply OnSearchFieldKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);
	void OnFilterTextChanged(const FText& Text);

	TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const;
	void OnMenuOpenChanged(bool bInOpen);

	FText GetFavoriteFontLabel() const;
	FText GetProjectFontLabel() const;
	FText GetSystemFontLabel() const;

	TSharedPtr<SListView<TSharedPtr<FString>>> FavoriteFontsListView;
	TSharedPtr<SListView<TSharedPtr<FString>>> ProjectFontsListView;
	TSharedPtr<SListView<TSharedPtr<FString>>> SystemFontsListView;

	TArray<TSharedPtr<FString>> FavoriteFontsItems;
	TArray<TSharedPtr<FString>> ProjectFontsItems;
	TArray<TSharedPtr<FString>> SystemFontItems;

	TSharedPtr<SBox> FavoriteSeparator;
	TSharedPtr<SBox> ProjectSeparator;
	TSharedPtr<SBox> SystemSeparator;
	
	TSharedPtr<STextBlock> FavoriteLabel;
	TSharedPtr<STextBlock> ProjectLabel;
	TSharedPtr<STextBlock> SystemLabel;
	
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SBox> FontContainer;
	TSharedPtr<SComboButton> ComboButton;

	TSharedPtr<IPropertyHandle> FontPropertyHandle;

	friend class UText3DEditorAccessHelper;
};
