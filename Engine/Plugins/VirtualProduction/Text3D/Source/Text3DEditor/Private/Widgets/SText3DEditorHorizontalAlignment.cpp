// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SText3DEditorHorizontalAlignment.h"

#include "PropertyHandle.h"
#include "Styles/Text3DEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Text3DTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "SText3DEditorHorizontalAlignment"

void SText3DEditorHorizontalAlignment::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;
	AlignmentChangedDelegate = InArgs._OnHorizontalAlignmentChanged;

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(0.f))
		[
			SNew(SGridPanel)
			+ SGridPanel::Slot(1, 0)
			[
				MakeAlignmentButton(EText3DHorizontalTextAlignment::Left, "Icons.Alignment.Left", LOCTEXT("AlignSelectedTextLeft", "Align Text to the Left"))
			]
			+ SGridPanel::Slot(2, 0)
			[
				MakeAlignmentButton(EText3DHorizontalTextAlignment::Center, "Icons.Alignment.Center_Y", LOCTEXT("AlignSelectedTextCenter", "Align Text to Center"))
			]
			+ SGridPanel::Slot(3, 0)
			[
				MakeAlignmentButton(EText3DHorizontalTextAlignment::Right, "Icons.Alignment.Right", LOCTEXT("AlignSelectedTextRight", "Align Text to the Right"))
			]
		]
	];

	PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SText3DEditorHorizontalAlignment::OnPropertyChanged));

}

TSharedRef<SButton> SText3DEditorHorizontalAlignment::MakeAlignmentButton(EText3DHorizontalTextAlignment InAlignment, FName InBrushName, const FText& InTooltip)
{
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SText3DEditorHorizontalAlignment::OnAlignmentButtonClicked, InAlignment)
		.ToolTipText(InTooltip)
		[
			SNew(SImage)
			.Image(FText3DEditorStyle::Get().GetBrush(InBrushName))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
			.ColorAndOpacity(this, &SText3DEditorHorizontalAlignment::GetButtonColorAndOpacity, InAlignment)
		];
}

FSlateColor SText3DEditorHorizontalAlignment::GetButtonColorAndOpacity(EText3DHorizontalTextAlignment InAlignment) const
{
	return InAlignment == GetPropertyAlignment() ?
		FSlateColor(EStyleColor::AccentBlue).GetSpecifiedColor() :
		FSlateColor(EStyleColor::Foreground).GetSpecifiedColor();
}

FReply SText3DEditorHorizontalAlignment::OnAlignmentButtonClicked(EText3DHorizontalTextAlignment InAlignment)
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetValue(static_cast<uint8>(InAlignment));
	}

	return FReply::Handled();
}

EText3DHorizontalTextAlignment SText3DEditorHorizontalAlignment::GetPropertyAlignment() const
{
	if (PropertyHandle.IsValid())
	{
		uint8 Value;
		PropertyHandle->GetValue(Value);
		return static_cast<EText3DHorizontalTextAlignment>(Value);
	}

	return EText3DHorizontalTextAlignment::Left;
}

void SText3DEditorHorizontalAlignment::OnPropertyChanged()
{
	AlignmentChangedDelegate.ExecuteIfBound(GetPropertyAlignment());
}

#undef LOCTEXT_NAMESPACE
