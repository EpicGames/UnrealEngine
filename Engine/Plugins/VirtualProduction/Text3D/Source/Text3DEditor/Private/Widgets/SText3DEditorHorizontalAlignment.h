// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"

class FReply;
class IPropertyHandle;
class SButton;
enum class EText3DHorizontalTextAlignment : uint8;
struct FSlateColor;

DECLARE_DELEGATE_OneParam(FOnHorizontalAlignmentChanged, EText3DHorizontalTextAlignment)

class SText3DEditorHorizontalAlignment : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SText3DEditorHorizontalAlignment)
		: _OnHorizontalAlignmentChanged()
	{}
		SLATE_EVENT(FOnHorizontalAlignmentChanged, OnHorizontalAlignmentChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

private:
	TSharedRef<SButton> MakeAlignmentButton(EText3DHorizontalTextAlignment InAlignment, FName InBrushName, const FText& InTooltip);
	FSlateColor GetButtonColorAndOpacity(EText3DHorizontalTextAlignment InAlignment) const;
	TEXT3DEDITOR_API FReply OnAlignmentButtonClicked(EText3DHorizontalTextAlignment InAlignment);
	EText3DHorizontalTextAlignment GetPropertyAlignment() const;
	void OnPropertyChanged();

	TSharedPtr<IPropertyHandle> PropertyHandle;
	FOnHorizontalAlignmentChanged AlignmentChangedDelegate;

	friend class UText3DEditorAccessHelper;
};
