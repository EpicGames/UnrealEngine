// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SText3DEditorVerticalAlignment.h"

#include "PropertyHandle.h"
#include "Styles/Text3DEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Text3DTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "SText3DEditorVerticalAlignment"

void SText3DEditorVerticalAlignment::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;
	AlignmentChangedDelegate = InArgs._OnVerticalAlignmentChanged;

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(0.f))
		[
			SNew(SGridPanel)
			+ SGridPanel::Slot(0, 0)
			[
				MakeAlignmentButton(EText3DVerticalTextAlignment::FirstLine, "Icons.Alignment.Top", LOCTEXT("AlignSelectedTextFirstLine", "Align Text to First Line"))
			]
			+ SGridPanel::Slot(1, 0)
			[
				MakeAlignmentButton(EText3DVerticalTextAlignment::Top, "Icons.Alignment.Top", LOCTEXT("AlignSelectedTextTop", "Align Text to Top"))
			]
			+ SGridPanel::Slot(2, 0)
			[
				MakeAlignmentButton(EText3DVerticalTextAlignment::Center, "Icons.Alignment.Center_Z", LOCTEXT("AlignSelectedTextCenter", "Align Text to Center"))
			]
			+ SGridPanel::Slot(3, 0)
			[
				MakeAlignmentButton(EText3DVerticalTextAlignment::Bottom, "Icons.Alignment.Bottom", LOCTEXT("AlignSelectedTextBottom", "Align Text to Bottom"))
			]
		]
	];

	PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SText3DEditorVerticalAlignment::OnPropertyChanged));
}

TSharedRef<SButton> SText3DEditorVerticalAlignment::MakeAlignmentButton(EText3DVerticalTextAlignment InAlignment, FName InBrushName, const FText& InTooltip)
{
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SText3DEditorVerticalAlignment::OnAlignmentButtonClicked, InAlignment)
		.ToolTipText(InTooltip)
		[
			SNew(SImage)
			.Image(FText3DEditorStyle::Get().GetBrush(InBrushName))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
			.ColorAndOpacity(this, &SText3DEditorVerticalAlignment::GetButtonColorAndOpacity, InAlignment)
		];
}

FSlateColor SText3DEditorVerticalAlignment::GetButtonColorAndOpacity(EText3DVerticalTextAlignment InAlignment) const
{
	return InAlignment == GetPropertyAlignment() ?
		FSlateColor(EStyleColor::AccentBlue).GetSpecifiedColor() :
		FSlateColor(EStyleColor::Foreground).GetSpecifiedColor();
}

FReply SText3DEditorVerticalAlignment::OnAlignmentButtonClicked(EText3DVerticalTextAlignment InAlignment)
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetValue(static_cast<uint8>(InAlignment));
	}

	return FReply::Handled();
}

EText3DVerticalTextAlignment SText3DEditorVerticalAlignment::GetPropertyAlignment() const
{
	if (PropertyHandle.IsValid())
	{
		uint8 Value;
		PropertyHandle->GetValue(Value);
		return static_cast<EText3DVerticalTextAlignment>(Value);
	}

	return EText3DVerticalTextAlignment::FirstLine;
}

void SText3DEditorVerticalAlignment::OnPropertyChanged()
{
	AlignmentChangedDelegate.ExecuteIfBound(GetPropertyAlignment());
}

#undef LOCTEXT_NAMESPACE