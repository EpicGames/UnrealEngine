// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FReply;
class IPropertyHandle;
class SButton;
enum class EText3DVerticalTextAlignment : uint8;
struct FSlateColor;

DECLARE_DELEGATE_OneParam(FOnVerticalAlignmentChanged, EText3DVerticalTextAlignment)

class SText3DEditorVerticalAlignment : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SText3DEditorVerticalAlignment)
		: _OnVerticalAlignmentChanged()
	{}
		SLATE_EVENT(FOnVerticalAlignmentChanged, OnVerticalAlignmentChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

private:
	TSharedRef<SButton> MakeAlignmentButton(EText3DVerticalTextAlignment InAlignment, FName InBrushName, const FText& InTooltip);
	FSlateColor GetButtonColorAndOpacity(EText3DVerticalTextAlignment InAlignment) const;
	TEXT3DEDITOR_API FReply OnAlignmentButtonClicked(EText3DVerticalTextAlignment InAlignment);
	EText3DVerticalTextAlignment GetPropertyAlignment() const;
	void OnPropertyChanged();

	TSharedPtr<IPropertyHandle> PropertyHandle;
	FOnVerticalAlignmentChanged AlignmentChangedDelegate;

	friend class UText3DEditorAccessHelper;
};