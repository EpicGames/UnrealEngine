// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class Text3DEditor : ModuleRules
{
	public Text3DEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateDependencyModuleNames.AddRange(new string[] {
			"AssetDefinition",
			"Core",
			"CoreUObject",
			"DeveloperSettings",
			"EditorFramework",
			"EditorSubsystem",
			"Engine",
			"FreeType2",
			"InputCore",
			"Projects",
			"PropertyEditor",
			"SlateCore",
			"Slate",
			"Text3D",
			"ToolMenus",
			"UnrealEd",
		});

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.Add("DirectX");

			PublicAdditionalLibraries.AddRange(new string[] {
				Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dwrite.lib")
			});
		}

		// Needed for FreeType/Font functionality
		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"zlib",
			"UElibPNG"
		);

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange(new string[] { "Foundation", "CoreText", "ApplicationServices"});
		}
	}
}
