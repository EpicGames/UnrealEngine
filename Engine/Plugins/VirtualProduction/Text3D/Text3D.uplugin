{
  "FileVersion": 3,
  "Version": 1,
  "FriendlyName": "Text 3D",
  "Description": "Tool to create 3D Text with advanced options",
  "Category": "Text",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "SupportURL": "",
  "CanContainContent": true,
  "Installed": false,
  "Modules": [
    {
      "Name": "Text3D",
      "Type": "Runtime",
      "LoadingPhase": "Default"
    },
	{
	  "Name":  "Text3DEditor",
	  "Type":  "Editor",
	  "LoadingPhase": "Default"
	}
  ],
  "Plugins": [
    {
      "Name": "GeometryProcessing",
      "Enabled": true
    },
	{
	  "Name": "GeometryScripting", 
	  "Enabled": true
	},
    {
      "Name": "GeometryMask",
      "Enabled": true
    }
  ]
}
